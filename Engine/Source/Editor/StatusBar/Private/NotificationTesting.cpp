// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "TimerManager.h"
#include "Editor.h"

#if (WITH_EDITOR || IS_PROGRAM) && !UE_BUILD_SHIPPING

struct FTestAsyncTask
{
	FProgressNotificationHandle ProgressHandle;
	FTimerHandle TimerHandle;
	int32 WorkDone;
};

static TArray<FTestAsyncTask> Tasks;
FTimerHandle TestTimerHandle;

static void StartTask(FString TaskName, int32 TotalWork, int32 IncAmount)
{
	FText ProgressNotification = FText::FromString(TaskName);

	FTestAsyncTask TestTask;
	TestTask.WorkDone = 0;
	TestTask.ProgressHandle = FSlateNotificationManager::Get().StartProgressNotification(ProgressNotification, TotalWork);

	int32 TaskIndex = Tasks.Num();

	GEditor->GetTimerManager()->SetTimer(
		TestTask.TimerHandle,
		FTimerDelegate::CreateLambda
		(
			[TaskIndex, TotalWork, IncAmount]()
			{
				FTestAsyncTask& CurrentTask = Tasks[TaskIndex];
				if (CurrentTask.WorkDone >= TotalWork)
				{
					GEditor->GetTimerManager()->ClearTimer(CurrentTask.TimerHandle);

				}
				else
				{
					CurrentTask.WorkDone += IncAmount;
					FSlateNotificationManager::Get().UpdateProgressNotification(CurrentTask.ProgressHandle, CurrentTask.WorkDone);
				}
			}
		),
		1.0f,
		true
	);

	Tasks.Add(MoveTemp(TestTask));
}

static void TestProgressBars()
{
	FString TaskName = FString::Printf(TEXT("Reticulating Splines %d"), Tasks.Num() + 1);
	int32 TotalWork = 10;

	StartTask(TaskName, TotalWork, 1);
}


FAutoConsoleCommand TestProgressBarsCommand(TEXT("Slate.TestProgressNotification"), TEXT(""), FConsoleCommandDelegate::CreateStatic(&TestProgressBars));

static void TestNotifications()
{
	static const float Timeout = 15.0f;

	int Delay = 0;

	// Text only
	FTSTicker::GetCoreTicker().AddTicker(TEXT("TestNotifications"), (float)Delay++, [](float DeltaTime)
		{
			FNotificationInfo NotificationInfo(FText::FromString(TEXT("Test Notification 1")));
			NotificationInfo.FadeInDuration = 2.0f;
			NotificationInfo.FadeOutDuration = 2.0f;
			NotificationInfo.ExpireDuration = Timeout;

			auto Notification = FSlateNotificationManager::Get().AddNotification(NotificationInfo);
			Notification->ExpireAndFadeout();

			return false;
		});

	// SubText 
	FTSTicker::GetCoreTicker().AddTicker(TEXT("TestNotifications"), (float)Delay++, [](float DeltaTime)
		{
			FNotificationInfo NotificationInfo(FText::FromString(TEXT("Test Notification 2")));
			NotificationInfo.FadeInDuration = 2.0f;
			NotificationInfo.FadeOutDuration = 2.0f;
			NotificationInfo.ExpireDuration = Timeout;

			NotificationInfo.SubText = FText::FromString(TEXT("SubText Test"));

			auto Notification = FSlateNotificationManager::Get().AddNotification(NotificationInfo);
			Notification->ExpireAndFadeout();

			return false;
		});

	// Throbber
	FTSTicker::GetCoreTicker().AddTicker(TEXT("TestNotifications"), (float)Delay++, [](float DeltaTime)
		{
			FNotificationInfo NotificationInfo(FText::FromString(TEXT("Test Notification 3")));
			NotificationInfo.FadeInDuration = 2.0f;
			NotificationInfo.FadeOutDuration = 2.0f;
			NotificationInfo.ExpireDuration = Timeout;

			NotificationInfo.bUseThrobber = true;
			auto Notification = FSlateNotificationManager::Get().AddNotification(NotificationInfo);
			Notification->SetCompletionState(SNotificationItem::CS_Pending);
			Notification->ExpireAndFadeout();

			return false;
		});

	// Success icon
	FTSTicker::GetCoreTicker().AddTicker(TEXT("TestNotifications"), (float)Delay++, [](float DeltaTime)
		{
			FNotificationInfo NotificationInfo(FText::FromString(TEXT("Test Notification 4")));
			NotificationInfo.FadeInDuration = 2.0f;
			NotificationInfo.FadeOutDuration = 2.0f;
			NotificationInfo.ExpireDuration = Timeout;

			NotificationInfo.bUseSuccessFailIcons = true;
			auto Notification = FSlateNotificationManager::Get().AddNotification(NotificationInfo);
			Notification->SetCompletionState(SNotificationItem::CS_Success);
			Notification->ExpireAndFadeout();

			return false;
		});

	// Failure icon
	FTSTicker::GetCoreTicker().AddTicker(TEXT("TestNotifications"), (float)Delay++, [](float DeltaTime)
		{
			FNotificationInfo NotificationInfo(FText::FromString(TEXT("Test Notification 5")));
			NotificationInfo.FadeInDuration = 2.0f;
			NotificationInfo.FadeOutDuration = 2.0f;
			NotificationInfo.ExpireDuration = Timeout;

			NotificationInfo.bUseSuccessFailIcons = true;
			auto Notification = FSlateNotificationManager::Get().AddNotification(NotificationInfo);
			Notification->SetCompletionState(SNotificationItem::CS_Fail);
			Notification->ExpireAndFadeout();

			return false;
		});

	// Checkbox
	FTSTicker::GetCoreTicker().AddTicker(TEXT("TestNotifications"), (float)Delay++, [](float DeltaTime)
		{
			FNotificationInfo NotificationInfo(FText::FromString(TEXT("Test Notification 6")));
			NotificationInfo.FadeInDuration = 2.0f;
			NotificationInfo.FadeOutDuration = 2.0f;
			NotificationInfo.ExpireDuration = Timeout;

			NotificationInfo.CheckBoxText = FText::FromString(TEXT("Don't ask again"));
			NotificationInfo.CheckBoxState = ECheckBoxState::Checked;
			NotificationInfo.CheckBoxStateChanged = FOnCheckStateChanged::CreateStatic([](ECheckBoxState NewState) {});

			auto Notification = FSlateNotificationManager::Get().AddNotification(NotificationInfo);

			Notification->ExpireAndFadeout();

			return false;
		});

	// Hyperlink
	FTSTicker::GetCoreTicker().AddTicker(TEXT("TestNotifications"), (float)Delay++, [](float DeltaTime)
		{
			FNotificationInfo NotificationInfo(FText::FromString(TEXT("Test Notification 7")));
			NotificationInfo.FadeInDuration = 2.0f;
			NotificationInfo.FadeOutDuration = 2.0f;
			NotificationInfo.ExpireDuration = Timeout;

			NotificationInfo.Hyperlink = FSimpleDelegate::CreateLambda([]() {});
			NotificationInfo.HyperlinkText = FText::FromString(TEXT("This is a hyperlink"));

			auto Notification = FSlateNotificationManager::Get().AddNotification(NotificationInfo);

			Notification->ExpireAndFadeout();

			return false;
		});

	// Buttons
	FTSTicker::GetCoreTicker().AddTicker(TEXT("TestNotifications"), (float)Delay++, [](float DeltaTime)
		{
			FNotificationInfo NotificationInfo(FText::FromString(TEXT("Test Notification 8")));
			NotificationInfo.FadeInDuration = 2.0f;
			NotificationInfo.FadeOutDuration = 2.0f;
			NotificationInfo.ExpireDuration = Timeout;

			FNotificationButtonInfo Button1(FText::FromString("OK"), FText::GetEmpty(), FSimpleDelegate(), SNotificationItem::ECompletionState::CS_None);
			FNotificationButtonInfo Button2(FText::FromString("CANCEL"), FText::GetEmpty(), FSimpleDelegate(), SNotificationItem::ECompletionState::CS_None);

			NotificationInfo.ButtonDetails.Add(Button1);
			NotificationInfo.ButtonDetails.Add(Button2);

			auto Notification = FSlateNotificationManager::Get().AddNotification(NotificationInfo);
			Notification->ExpireAndFadeout();

			return false;
		});

	// Copy to Clipboard
	FTSTicker::GetCoreTicker().AddTicker(TEXT("TestNotifications"), (float)Delay++, [](float DeltaTime)
		{
			FNotificationInfo NotificationInfo(FText::FromString(TEXT("Test Notification 9")));
			NotificationInfo.FadeInDuration = 2.0f;
			NotificationInfo.FadeOutDuration = 2.0f;
			NotificationInfo.ExpireDuration = Timeout;

			NotificationInfo.bUseCopyToClipboard = true;

			auto Notification = FSlateNotificationManager::Get().AddNotification(NotificationInfo);
			Notification->ExpireAndFadeout();

			return false;
		});

	// Everything
	FTSTicker::GetCoreTicker().AddTicker(TEXT("TestNotifications"), (float)Delay++, [](float DeltaTime)
		{
			FNotificationInfo NotificationInfo(FText::FromString(TEXT("Everything Under The Sun. This one also has a lot of text which should wrap to the next line")));
			NotificationInfo.FadeInDuration = 2.0f;
			NotificationInfo.FadeOutDuration = 2.0f;
			NotificationInfo.ExpireDuration = Timeout;

			NotificationInfo.SubText = FText::FromString(TEXT("SubText Test"));

			NotificationInfo.CheckBoxText = FText::FromString(TEXT("Don't ask again"));
			NotificationInfo.CheckBoxState = ECheckBoxState::Checked;
			NotificationInfo.CheckBoxStateChanged = FOnCheckStateChanged::CreateStatic([](ECheckBoxState NewState) {});

			NotificationInfo.Hyperlink = FSimpleDelegate::CreateLambda([]() {});
			NotificationInfo.HyperlinkText = FText::FromString(TEXT("This is a hyperlink"));

			NotificationInfo.bUseSuccessFailIcons = true;
			NotificationInfo.bUseThrobber = true;
			NotificationInfo.bUseCopyToClipboard = true;

			FNotificationButtonInfo Button1(FText::FromString("OK"), FText::GetEmpty(), FSimpleDelegate());
			FNotificationButtonInfo Button2(FText::FromString("CANCEL"), FText::GetEmpty(), FSimpleDelegate());

			NotificationInfo.ButtonDetails.Add(Button1);
			NotificationInfo.ButtonDetails.Add(Button2);


			auto Notification = FSlateNotificationManager::Get().AddNotification(NotificationInfo);
			Notification->SetCompletionState(SNotificationItem::CS_Pending);

			Notification->ExpireAndFadeout();

			return false;
		});

	// Everything overflowing
	FTSTicker::GetCoreTicker().AddTicker(TEXT("TestNotifications"), (float)Delay++, [](float DeltaTime)
		{
			FNotificationInfo NotificationInfo(FText::FromString(TEXT("This one has a lot of text on the buttons which should overflow properly")));
			NotificationInfo.FadeInDuration = 2.0f;
			NotificationInfo.FadeOutDuration = 2.0f;
			NotificationInfo.ExpireDuration = Timeout;

			NotificationInfo.SubText = FText::FromString(TEXT("This one also has a lot of subtext which should wrap to the next line"));

			NotificationInfo.CheckBoxText = FText::FromString(TEXT("This is a checkbox with a lot of text. Hover over it to read the full text in the tooltip."));
			NotificationInfo.CheckBoxState = ECheckBoxState::Checked;
			NotificationInfo.CheckBoxStateChanged = FOnCheckStateChanged::CreateStatic([](ECheckBoxState NewState) {});

			NotificationInfo.Hyperlink = FSimpleDelegate::CreateLambda([]() {});
			NotificationInfo.HyperlinkText = FText::FromString(TEXT("This is a hyperlink with a lot of text. Hover over it to read the full text in the tooltip."));

			NotificationInfo.bUseSuccessFailIcons = true;
			NotificationInfo.bUseThrobber = true;
			NotificationInfo.bUseCopyToClipboard = true;

			FNotificationButtonInfo Button1(FText::FromString("This is a button with a lot of text. Hover over it to read the full text in the tooltip."), FText::GetEmpty(), FSimpleDelegate());
			FNotificationButtonInfo Button2(FText::FromString("This is another button with a lot of text. Hover over it to read the full text in the tooltip."), FText::GetEmpty(), FSimpleDelegate());

			NotificationInfo.ButtonDetails.Add(Button1);
			NotificationInfo.ButtonDetails.Add(Button2);

			auto Notification = FSlateNotificationManager::Get().AddNotification(NotificationInfo);
			Notification->SetCompletionState(SNotificationItem::CS_Pending);

			Notification->ExpireAndFadeout();

			return false;
		});
}

FAutoConsoleCommand TestNotificationCommand(TEXT("Slate.TestNotifications"), TEXT(""), FConsoleCommandDelegate::CreateStatic(&TestNotifications));

#endif // (WITH_EDITOR || IS_PROGRAM) && !UE_BUILD_SHIPPING
