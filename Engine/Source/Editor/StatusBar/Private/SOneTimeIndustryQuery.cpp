// Copyright Epic Games, Inc. All Rights Reserved.

#include "SOneTimeIndustryQuery.h"
#include "Widgets/Notifications/SNotificationBackground.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboBox.h"
#include "SPrimaryButton.h"
#include "EngineAnalytics.h"
#include "Misc/EngineBuildSettings.h"
#include "Editor/EditorEngine.h"
#include "Analytics/AnalyticsPrivacySettings.h"

#define LOCTEXT_NAMESPACE "OneTimeIndustryQuery"

TWeakPtr<SOneTimeIndustryQuery> SOneTimeIndustryQuery::ActiveNotification;
TWeakPtr<SWindow> SOneTimeIndustryQuery::ParentWindow;

const FName SOneTimeIndustryQuery::GamesIndustryName = FName("Games");
const FName SOneTimeIndustryQuery::FilmIndustryName = FName("FilmAndTV");
const FName SOneTimeIndustryQuery::ArchitectureIndustryName = FName("Architecture");
const FName SOneTimeIndustryQuery::AutoIndustryName = FName("AutomotiveAndManufacturing");
const FName SOneTimeIndustryQuery::BroadcastIndustryName = FName("BroadcastingAndLiveEvents");
const FName SOneTimeIndustryQuery::AdIndustryName = FName("AdvertisingAndMarketing");
const FName SOneTimeIndustryQuery::SimulationIndustryName = FName("TrainingAndSimulation");
const FName SOneTimeIndustryQuery::FashionIndustryName = FName("Fashion");
const FName SOneTimeIndustryQuery::OtherIndustryName = FName("Other");

void SOneTimeIndustryQuery::Show(TSharedPtr<SWindow> InParentWindow)
{
	if (!ActiveNotification.IsValid())
	{
		TSharedRef<SOneTimeIndustryQuery> ActiveNotificationRef =
			SNew(SOneTimeIndustryQuery);

		ActiveNotification = ActiveNotificationRef;
		ParentWindow = InParentWindow;
		InParentWindow->AddOverlaySlot()
			.VAlign(VAlign_Bottom)
			.HAlign(HAlign_Right)
			.Padding(FMargin(20.0f, 20.0f, 10.0f, 50.f))
			[
				ActiveNotificationRef
			];
	}
}

void SOneTimeIndustryQuery::Dismiss()
{
	TSharedPtr<SOneTimeIndustryQuery> ActiveNotificationPin = ActiveNotification.Pin();
	if (ParentWindow.IsValid() && ActiveNotificationPin.IsValid())
	{
		ParentWindow.Pin()->RemoveOverlaySlot(ActiveNotificationPin.ToSharedRef());
	}

	ParentWindow.Reset();

	ActiveNotification.Reset();
}

const bool SOneTimeIndustryQuery::ShouldShowIndustryQuery()
{
	return !FEngineBuildSettings::IsInternalBuild()  
		&& GEngine->AreEditorAnalyticsEnabled()
		&& !GetDefault<UAnalyticsPrivacySettings>()->bSuppressIndustryPopup;
}

void SOneTimeIndustryQuery::Construct(const FArguments& InArgs)
{
	IndustryComboList.Add(MakeShared<FName>(SOneTimeIndustryQuery::GamesIndustryName));
	IndustryComboList.Add(MakeShared<FName>(SOneTimeIndustryQuery::FilmIndustryName));
	IndustryComboList.Add(MakeShared<FName>(SOneTimeIndustryQuery::ArchitectureIndustryName));
	IndustryComboList.Add(MakeShared<FName>(SOneTimeIndustryQuery::AutoIndustryName));
	IndustryComboList.Add(MakeShared<FName>(SOneTimeIndustryQuery::BroadcastIndustryName));
	IndustryComboList.Add(MakeShared<FName>(SOneTimeIndustryQuery::AdIndustryName));
	IndustryComboList.Add(MakeShared<FName>(SOneTimeIndustryQuery::SimulationIndustryName));
	IndustryComboList.Add(MakeShared<FName>(SOneTimeIndustryQuery::FashionIndustryName));
	IndustryComboList.Add(MakeShared<FName>(SOneTimeIndustryQuery::OtherIndustryName));

	ChildSlot
		[
			SNew(SBox)
			.WidthOverride(375.0f)
			.HeightOverride(128.0f)
			[
				SNew(SNotificationBackground)
				.Padding(FMargin(8, 8))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.Padding(20.0f, 15.0f, 0.0f, 0.0f)
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 0.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
							.Font(FAppStyle::Get().GetFontStyle("NotificationList.FontBold"))
							.Text(LOCTEXT("IndustryQuestionDesc", "Help us improve Unreal Engine by \ntelling us your industry."))
							.ColorAndOpacity(FStyleColors::ForegroundHover)
						]

						+ SVerticalBox::Slot()
						.VAlign(VAlign_Center)
						.FillHeight(1.0f)
						.Padding(0.0f, 12.0f, 0.0f, 12.0f)
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.FillWidth(1.0f)
							[
								SNew(SComboBox<TSharedPtr<FName>>)
								.OptionsSource(&IndustryComboList)
								.OnGenerateWidget(this, &SOneTimeIndustryQuery::GenerateIndustryComboItem)
								.OnSelectionChanged(this, &SOneTimeIndustryQuery::HandleIndustryComboChanged)
								[
									SNew(STextBlock)
									.Text(this, &SOneTimeIndustryQuery::GetUserSetIndustryNameText)
								]
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(15.0f, 0.0f, 0.0f, 0.0f)
							[
								SNew(SPrimaryButton)
								.Text(LOCTEXT("IndustryFeedbackSubmit", "Submit"))
								.IsEnabled(this, &SOneTimeIndustryQuery::CanSubmitIndustryInfo)
								.OnClicked(this, &SOneTimeIndustryQuery::OnSubmit)
							]
						]
					]

					+ SHorizontalBox::Slot()
					.Padding(0.0f, 0.0f, 0.0f, 0.0f)
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Top)
					.AutoWidth()
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.OnClicked_Lambda([]() { SOneTimeIndustryQuery::Dismiss(); return FReply::Handled(); })
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.X"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
				]
			]
		];

}

bool SOneTimeIndustryQuery::CanSubmitIndustryInfo() const
{
	return UserSetIndustry.IsSet();
}

void SOneTimeIndustryQuery::HandleIndustryComboChanged(TSharedPtr<FName> Item, ESelectInfo::Type SelectInfo)
{
	FSlateApplication::Get().DismissAllMenus();

	UserSetIndustry = *Item.Get();
}

TSharedRef<SWidget> SOneTimeIndustryQuery::GenerateIndustryComboItem(TSharedPtr<FName> InItem) const
{
	return SNew(STextBlock).Text(GetIndustryNameText(*InItem.Get()));
}

FText SOneTimeIndustryQuery::GetUserSetIndustryNameText() const
{
	if (!UserSetIndustry.IsSet())
	{
		return LOCTEXT("SelectYourIndustry", "Select Your Industry");
	}
	return GetIndustryNameText(UserSetIndustry.GetValue());
}

FText SOneTimeIndustryQuery::GetIndustryNameText(FName IndustryName) const
{
	if (IndustryName == SOneTimeIndustryQuery::GamesIndustryName)
	{
		return LOCTEXT("GameDevelopment", "Game Development");
	}
	else if (IndustryName == SOneTimeIndustryQuery::FilmIndustryName)
	{
		return LOCTEXT("FilmAndTV", "Film & TV");
	}
	else if (IndustryName == SOneTimeIndustryQuery::ArchitectureIndustryName)
	{
		return LOCTEXT("Architecture", "Architecture");
	}
	else if (IndustryName == SOneTimeIndustryQuery::AutoIndustryName)
	{
		return LOCTEXT("AutomotiveAndManufacturing", "Automotive & Manufacturing");
	}
	else if (IndustryName == SOneTimeIndustryQuery::BroadcastIndustryName)
	{
		return LOCTEXT("BroadcastingAndLiveEvents", "Broadcasting & Live Events");
	}
	else if (IndustryName == SOneTimeIndustryQuery::AdIndustryName)
	{
		return LOCTEXT("AdvertisingAndMarketing", "Advertising & Marketing");
	}
	else if (IndustryName == SOneTimeIndustryQuery::SimulationIndustryName)
	{
		return LOCTEXT("TrainingAndSimulation", "Training & Simulation");
	}
	else if (IndustryName == SOneTimeIndustryQuery::FashionIndustryName)
	{
		return LOCTEXT("Fashion", "Fashion");
	}
	else if (IndustryName == SOneTimeIndustryQuery::OtherIndustryName)
	{
		return LOCTEXT("Other", "Other");
	}
	return FText::GetEmpty();
}

FReply SOneTimeIndustryQuery::OnSubmit()
{
	if (FEngineAnalytics::IsAvailable())
	{
		TArray< FAnalyticsEventAttribute > IndustryAttribs;
		IndustryAttribs.Add(FAnalyticsEventAttribute(TEXT("Industry"), UserSetIndustry.GetValue().ToString()));
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.UserIndustry"), IndustryAttribs);
	}
	SOneTimeIndustryQuery::Dismiss();
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
