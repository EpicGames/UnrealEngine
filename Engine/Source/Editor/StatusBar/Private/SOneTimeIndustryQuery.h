// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Styling/StyleColors.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class SWindow;

class SOneTimeIndustryQuery : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SOneTimeIndustryQuery)
	{}
	SLATE_END_ARGS()

	SOneTimeIndustryQuery()
	: NewBadgeBrush(FStyleColors::Success)
	, KeybindBackgroundBrush(FLinearColor::Transparent, 6.0f, FStyleColors::ForegroundHover, 1.5f)
	{}

	static void Show(TSharedPtr<SWindow> InParentWindow);

	static void Dismiss();

	static const bool ShouldShowIndustryQuery();

	void Construct(const FArguments& InArgs);

private:
	bool CanSubmitIndustryInfo() const;
	void HandleIndustryComboChanged(TSharedPtr<FName> Item, ESelectInfo::Type SelectInfo);
	TSharedRef<SWidget> GenerateIndustryComboItem(TSharedPtr<FName> InItem) const;
	FText GetUserSetIndustryNameText() const;
	FText GetIndustryNameText(FName IndustryName) const;
	FReply OnSubmit();

private:
	FSlateRoundedBoxBrush NewBadgeBrush;
	FSlateRoundedBoxBrush KeybindBackgroundBrush;
	static TWeakPtr<SOneTimeIndustryQuery> ActiveNotification;
	static TWeakPtr<SWindow> ParentWindow;
	TArray<TSharedPtr<FName>> IndustryComboList;
	TOptional<FName> UserSetIndustry;

	static const FName GamesIndustryName;
	static const FName FilmIndustryName;
	static const FName ArchitectureIndustryName;
	static const FName AutoIndustryName;
	static const FName BroadcastIndustryName;
	static const FName AdIndustryName;
	static const FName SimulationIndustryName;
	static const FName FashionIndustryName;
	static const FName OtherIndustryName;
};


