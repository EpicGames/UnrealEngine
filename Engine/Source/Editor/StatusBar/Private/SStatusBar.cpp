// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStatusBar.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SSeparator.h"
#include "Framework/Docking/TabManager.h"
#include "TimerManager.h"
#include "Framework/Commands/Commands.h"
#include "ToolMenuContext.h"
#include "ToolMenus.h"
#include "SourceControlMenuHelpers.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SSplitter.h"
#include "InputCoreTypes.h"
#include "SWidgetDrawer.h"
#include "Misc/ConfigCacheIni.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Notifications/INotificationWidget.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "StatusBar"

namespace StatusBarNotificationConstants
{
	// Delay before a progress notification becomes visible. This is to avoid the status bar to animate and flicker from short lived notifications. 
	const double NotificationDelay = 3.0;
}

class SDrawerOverlay : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SDrawerOverlay)
	{
		_Clipping = EWidgetClipping::ClipToBounds;
		_ShadowOffset = FVector2D(10.0f, 20.0f);
	}
		SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_ARGUMENT(float, MinDrawerHeight)
		SLATE_ARGUMENT(float, MaxDrawerHeight)
		SLATE_ARGUMENT(float, TargetDrawerHeight)
		SLATE_EVENT(FOnStatusBarDrawerTargetHeightChanged, OnTargetHeightChanged)
		SLATE_EVENT(FSimpleDelegate, OnDismissComplete)
		SLATE_ARGUMENT(FVector2D, ShadowOffset)
	SLATE_END_ARGS()

	~SDrawerOverlay()
	{
		FSlateThrottleManager::Get().LeaveResponsiveMode(AnimationThrottle);
	}

	void Construct(const FArguments& InArgs)
	{
		CurrentHeight = 0;

		ShadowOffset = InArgs._ShadowOffset;
		ExpanderSize = 5.0f;

		SplitterStyle = &FAppStyle::Get().GetWidgetStyle<FSplitterStyle>("Splitter");

		MinHeight = InArgs._MinDrawerHeight;

		MaxHeight = InArgs._MaxDrawerHeight;

		TargetHeight = FMath::Clamp(InArgs._TargetDrawerHeight, MinHeight, MaxHeight);

		OnTargetHeightChanged = InArgs._OnTargetHeightChanged;

		BackgroundBrush = FAppStyle::Get().GetBrush("StatusBar.DrawerBackground");
		ShadowBrush = FAppStyle::Get().GetBrush("StatusBar.DrawerShadow");
		BorderBrush = FAppStyle::Get().GetBrush("Docking.Sidebar.Border");

		bIsResizeHandleHovered = false;
		bIsResizing = false;

		OnDismissComplete = InArgs._OnDismissComplete;

		DrawerEasingCurve = FCurveSequence(0.0f, 0.15f, ECurveEaseFunction::QuadOut);

		ChildSlot
		[
			InArgs._Content.Widget
		];
	}

	void UpdateHeightInterp(float InAlpha)
	{
		float NewHeight = FMath::Lerp(0.0f, TargetHeight, InAlpha);

		SetHeight(NewHeight);
	}

	virtual bool SupportsKeyboardFocus() const override
	{
		return true;
	}

	virtual FVector2D ComputeDesiredSize(float) const
	{
		return FVector2D(1.0f, TargetHeight + ShadowOffset.Y);
	}

	virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override
	{
		const EVisibility ChildVisibility = ChildSlot.GetWidget()->GetVisibility();
		if (ArrangedChildren.Accepts(ChildVisibility))
		{
			ArrangedChildren.AddWidget(
				AllottedGeometry.MakeChild(
					ChildSlot.GetWidget(),
					ShadowOffset,
					FVector2D(AllottedGeometry.GetLocalSize().X - (ShadowOffset.X * 2), TargetHeight)
				)
			);
		}
	}

	virtual FReply OnMouseButtonDown(const FGeometry& AllottedGeometry, const FPointerEvent& MouseEvent) override
	{
		FReply Reply = FReply::Unhandled();
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			const FGeometry RenderTransformedChildGeometry = GetRenderTransformedGeometry(AllottedGeometry);
			const FGeometry ResizeHandleGeometry = GetResizeHandleGeometry(AllottedGeometry);

			if (ResizeHandleGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition()))
			{
				bIsResizing = true;
				InitialResizeGeometry = ResizeHandleGeometry;
				InitialHeightAtResize = CurrentHeight;
				ResizeThrottleHandle = FSlateThrottleManager::Get().EnterResponsiveMode();

				Reply = FReply::Handled().CaptureMouse(SharedThis(this));
			}
		}

		return Reply;

	}

	FReply OnMouseButtonUp(const FGeometry& AllottedGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && bIsResizing == true)
		{
			bIsResizing = false;
			FSlateThrottleManager::Get().LeaveResponsiveMode(ResizeThrottleHandle);

			OnTargetHeightChanged.ExecuteIfBound(TargetHeight);
			return FReply::Handled().ReleaseMouseCapture();
		}
		return FReply::Unhandled();
	}

	FReply OnMouseMove(const FGeometry& AllottedGeometry, const FPointerEvent& MouseEvent) override
	{
		const FGeometry ResizeHandleGeometry = GetResizeHandleGeometry(AllottedGeometry);

		bIsResizeHandleHovered = ResizeHandleGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition());

		if (bIsResizing && this->HasMouseCapture() && !MouseEvent.GetCursorDelta().IsZero())
		{
			const FVector2f LocalMousePos = InitialResizeGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
			const float DeltaHeight = (InitialResizeGeometry.GetLocalPositionAtCoordinates(FVector2f::ZeroVector) - LocalMousePos).Y;

			TargetHeight = FMath::Clamp(InitialHeightAtResize + DeltaHeight, MinHeight, MaxHeight);
			SetHeight(InitialHeightAtResize + DeltaHeight);


			return FReply::Handled();
		}
		else
		{
			return FReply::Unhandled();
		}
	}

	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override
	{
		SCompoundWidget::OnMouseLeave(MouseEvent);

		bIsResizeHandleHovered = false;
	}

	FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override
	{
		return bIsResizing || bIsResizeHandleHovered ? FCursorReply::Cursor(EMouseCursor::ResizeUpDown) : FCursorReply::Unhandled();
	}

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		static FSlateColor ShadowColor = FAppStyle::Get().GetSlateColor("Colors.Foldout");

		const FGeometry RenderTransformedChildGeometry = GetRenderTransformedGeometry(AllottedGeometry);
		const FGeometry ResizeHandleGeometry = GetResizeHandleGeometry(AllottedGeometry);

		// Draw the resize handle
		if (bIsResizing || bIsResizeHandleHovered)
		{
			const FSlateBrush* SplitterBrush = &SplitterStyle->HandleHighlightBrush;
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				ResizeHandleGeometry.ToPaintGeometry(),
				SplitterBrush,
				ESlateDrawEffect::None,
				SplitterBrush->GetTint(InWidgetStyle));
		}

		// Top Shadow
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			RenderTransformedChildGeometry.ToPaintGeometry(),
			ShadowBrush,
			ESlateDrawEffect::None,
			ShadowBrush->GetTint(InWidgetStyle));

		// Background
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			RenderTransformedChildGeometry.ToPaintGeometry(FVector2f(AllottedGeometry.GetLocalSize().X - (ShadowOffset.X * 2), TargetHeight), FSlateLayoutTransform(ShadowOffset)),
			BackgroundBrush,
			ESlateDrawEffect::None,
			BackgroundBrush->GetTint(InWidgetStyle));

		int32 OutLayerId = SCompoundWidget::OnPaint(Args, RenderTransformedChildGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

		// Bottom shadow
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			OutLayerId,
			AllottedGeometry.ToPaintGeometry(FVector2f(AllottedGeometry.GetLocalSize().X, ShadowOffset.Y), FSlateLayoutTransform(FVector2f(0.0f, AllottedGeometry.GetLocalSize().Y - ShadowOffset.Y))),
			ShadowBrush,
			ESlateDrawEffect::None,
			ShadowBrush->GetTint(InWidgetStyle));


		FSlateDrawElement::MakeBox(
			OutDrawElements,
			OutLayerId+1,
			RenderTransformedChildGeometry.ToPaintGeometry(FVector2f(AllottedGeometry.GetLocalSize().X - (ShadowOffset.X * 2), TargetHeight), FSlateLayoutTransform(ShadowOffset)),
			BorderBrush,
			ESlateDrawEffect::None,
			BorderBrush->GetTint(InWidgetStyle));

		return OutLayerId+1;

	}

	void Open()
	{
		DrawerEasingCurve.Play(AsShared(), false, DrawerEasingCurve.IsPlaying() ? DrawerEasingCurve.GetSequenceTime() : 0.0f, false);

		if (!DrawerOpenCloseTimer.IsValid())
		{
			AnimationThrottle = FSlateThrottleManager::Get().EnterResponsiveMode();
			DrawerOpenCloseTimer = RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SDrawerOverlay::UpdateDrawerAnimation));
		}
	}

	void Dismiss()
	{
		if (DrawerEasingCurve.IsForward())
		{
			DrawerEasingCurve.Reverse();
		}

		if (!DrawerOpenCloseTimer.IsValid())
		{
			AnimationThrottle = FSlateThrottleManager::Get().EnterResponsiveMode();
			DrawerOpenCloseTimer = RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SDrawerOverlay::UpdateDrawerAnimation));
		}
	}
private:
	FGeometry GetRenderTransformedGeometry(const FGeometry& AllottedGeometry) const
	{
		return AllottedGeometry.MakeChild(FSlateRenderTransform(FVector2D(0.0f, TargetHeight - CurrentHeight)));
	}

	FGeometry GetResizeHandleGeometry(const FGeometry& AllottedGeometry) const
	{
		return GetRenderTransformedGeometry(AllottedGeometry).MakeChild(FVector2D(AllottedGeometry.GetLocalSize().X-ShadowOffset.X*2, ExpanderSize), FSlateLayoutTransform(ShadowOffset - FVector2D(0.0f, ExpanderSize)));
	}

	void SetHeight(float NewHeight)
	{
		CurrentHeight = FMath::Clamp(NewHeight, MinHeight, TargetHeight);
	}

	EActiveTimerReturnType UpdateDrawerAnimation(double CurrentTime, float DeltaTime)
	{
		UpdateHeightInterp(DrawerEasingCurve.GetLerp());

		if (!DrawerEasingCurve.IsPlaying())
		{
			if (DrawerEasingCurve.IsAtStart())
			{
				OnDismissComplete.ExecuteIfBound();
			}

			FSlateThrottleManager::Get().LeaveResponsiveMode(AnimationThrottle);
			DrawerOpenCloseTimer.Reset();
			return EActiveTimerReturnType::Stop;
		}

		return EActiveTimerReturnType::Continue;
	}


private:
	FGeometry InitialResizeGeometry;
	TSharedPtr<FActiveTimerHandle> DrawerOpenCloseTimer;
	FOnStatusBarDrawerTargetHeightChanged OnTargetHeightChanged;
	FCurveSequence DrawerEasingCurve;
	FSimpleDelegate OnDismissComplete;
	const FSlateBrush* BackgroundBrush;
	const FSlateBrush* ShadowBrush;
	const FSlateBrush* BorderBrush;
	const FSplitterStyle* SplitterStyle;
	FVector2D ShadowOffset;
	FThrottleRequest AnimationThrottle;
	FThrottleRequest ResizeThrottleHandle;
	float ExpanderSize;
	float CurrentHeight;
	float MinHeight;
	float MaxHeight;
	float TargetHeight; 
	float InitialHeightAtResize;
	bool bIsResizing;
	bool bIsResizeHandleHovered;
};

class SStatusBarProgressWidget : public SCompoundWidget, public INotificationWidget
{
public:
	SLATE_BEGIN_ARGS(SStatusBarProgressWidget)
	{}
		SLATE_ATTRIBUTE(const FStatusBarProgress*, StatusBarProgress)
	SLATE_END_ARGS()
	
public:
	void Construct(const FArguments& InArgs, bool bIsShownInNotification = false)
	{
		StatusBarProgress = InArgs._StatusBarProgress;

		ChildSlot
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 3.0f, 0.0f, 2.0f)
				[
					SAssignNew(ProgressTextWidget, STextBlock)
				]

				+ SVerticalBox::Slot()
				//.AutoWidth()
				[
					SNew(SBox)
					.HeightOverride(8.f)
					[
						SNew(SOverlay)
						+ SOverlay::Slot()
						.VAlign(VAlign_Center)
						.Padding(1.0f, 0.0f)
						[
							SAssignNew(ProgressBar, SProgressBar)
							.Percent(0.0f)
						]
						+ SOverlay::Slot()
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("StatusBar.ProgressOverlay"))
							.Visibility(EVisibility::HitTestInvisible)
						]
					]
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SAssignNew(PercentText, STextBlock)
			]
		];
	
		if (bIsShownInNotification)
		{
			ProgressTextWidget->SetFont(FAppStyle::Get().GetFontStyle("NotificationList.FontBold"));
		}
	}

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		const FStatusBarProgress* ProgressData = StatusBarProgress.Get();
		if (ProgressData)
		{
			float PercentDone = FMath::Clamp((float)ProgressData->TotalWorkDone / ProgressData->TotalWorkToDo, 0.0f, 1.0f);
			ProgressTextWidget->SetText(ProgressData->DisplayText);
			PercentText->SetText(FText::AsPercent(PercentDone));
			ProgressBar->SetPercent(PercentDone);
		}
		else if(StatusBarProgress.IsBound())
		{
			StatusBarProgress = nullptr;
			FText CurrentText = ProgressTextWidget->GetText();
			ProgressTextWidget->SetText(FText::Format(LOCTEXT("CancelledProgressText", "{0} (Canceled)"), CurrentText));
		}
	}

	void SetProgressText(FText ProgressText)
	{
		ProgressTextWidget->SetText(ProgressText);
	}

	void SetProgressPercent(float Percent)
	{
		ProgressBar->SetPercent(Percent);
		PercentText->SetText(FText::AsPercent(Percent));
	}
	
	/** INotificationWidget interface */
	virtual void OnSetCompletionState(SNotificationItem::ECompletionState) override
	{}

	TSharedRef<SWidget> AsWidget() override
	{
		return AsShared();
	}
private:
	TAttribute<const FStatusBarProgress*> StatusBarProgress;
	TSharedPtr<SProgressBar> ProgressBar;
	TSharedPtr<STextBlock> PercentText;
	TSharedPtr<STextBlock> ProgressTextWidget;
};

class SStatusBarProgressArea : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SStatusBarProgressArea)
	{}
		SLATE_EVENT(FOnGetContent, OnGetProgressMenuContent)

	SLATE_END_ARGS()

	void SetPercent(float Percent)
	{
		MainProgressWidget->SetProgressPercent(Percent);
		MainProgressWidget->SetProgressText(FText::AsPercent(Percent));
	}

	void SetProgressText(FText ProgressText)
	{
		MainProgressWidget->SetProgressText(ProgressText);
	}

public:
	void Construct(const FArguments& InArgs)
	{
		OpenCloseEasingCurve = FCurveSequence(0.0f, 0.15f, ECurveEaseFunction::QuadOut);

		SetVisibility(EVisibility::Collapsed);

		ChildSlot
		[
			SAssignNew(Box, SBox)
			.WidthOverride(300.0f)
			.Padding(FMargin(4.0f,0.0f))
			[
				SAssignNew(ProgressCombo, SComboButton)
				.MenuPlacement(MenuPlacement_AboveAnchor)
				.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
				.OnGetMenuContent(InArgs._OnGetProgressMenuContent)
				.ButtonContent()
				[
					SAssignNew(MainProgressWidget, SStatusBarProgressWidget)
				]
			]
		];
	}

	void OpenProgressBar()
	{
		if(!GetVisibility().IsVisible())
		{
			SetVisibility(EVisibility::Visible);

			if (!OpenCloseEasingCurve.IsPlaying())
			{
				OpenCloseEasingCurve.Play(SharedThis(this), false, 0.0f, false);

				if (!OpenCloseTimer.IsValid())
				{
					OpenCloseTimer = RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateSP(SharedThis(this), &SStatusBarProgressArea::UpdateProgressAnimation));
				}
			}
		}
	}

	void DismissProgressBar()
	{
		if (GetVisibility().IsVisible())
		{
			if (OpenCloseEasingCurve.IsForward())
			{
				OpenCloseEasingCurve.Reverse();
			}

			if (!OpenCloseTimer.IsValid())
			{
				OpenCloseTimer = RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SStatusBarProgressArea::UpdateProgressAnimation));
			}

			ProgressCombo->SetIsOpen(false);
		}
	}

	virtual FVector2D ComputeDesiredSize(float Scale) const
	{
		return SCompoundWidget::ComputeDesiredSize(Scale) * FVector2D(OpenCloseEasingCurve.GetLerp(), 1.0f);
	}
private:
	EActiveTimerReturnType UpdateProgressAnimation(double CurrentTime, float DeltaTime)
	{
		if (!OpenCloseEasingCurve.IsPlaying())
		{
			if (OpenCloseEasingCurve.IsAtStart())
			{
				SetVisibility(EVisibility::Collapsed);
			}

			OpenCloseTimer.Reset();
			return EActiveTimerReturnType::Stop;
		}

		return EActiveTimerReturnType::Continue;
	}

private:
	TSharedPtr<SBox> Box;
	TSharedPtr<SStatusBarProgressWidget> MainProgressWidget;
	TSharedPtr<SComboButton> ProgressCombo;
	FCurveSequence OpenCloseEasingCurve;
	FThrottleRequest AnimationThrottle;
	TSharedPtr<FActiveTimerHandle> OpenCloseTimer;
};

TUniquePtr<FSlateRoundedBoxBrush> SStatusBar::SummonPanelDrawerRoundedBox;

SStatusBar::~SStatusBar()
{
	// Ensure the content browser is removed if we're being destroyed
	CloseDrawerImmediately();
}

void SStatusBar::Construct(const FArguments& InArgs, FName InStatusBarName, const TSharedRef<SDockTab> InParentTab, TArray<UStatusBarSubsystem::FTabIdAndButtonLabel>&& PanelDrawerSummons)
{
	ParentTab = InParentTab;
	bCreateSourceControlSection = InArgs._CreateSourceControlSection;

	UpArrow = FAppStyle::Get().GetBrush("StatusBar.ContentBrowserUp");
	DownArrow = FAppStyle::Get().GetBrush("StatusBar.ContentBrowserDown");

	const FSlateBrush* StatusBarBackground = FAppStyle::Get().GetBrush("Brushes.Panel");

	TSharedPtr<SHorizontalBox> MainContentBox;
	TSharedPtr<SWidget> SummonPanelDrawerWidget = MakeSummonPanelDrawer(MoveTemp(PanelDrawerSummons));

	ChildSlot
	[
		SNew(SBox)
		.HeightOverride(FAppStyle::Get().GetFloat("StatusBar.Height"))
		[	
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(WidgetDrawer, SWidgetDrawer, InStatusBarName)
			]
			+SHorizontalBox::Slot()
			[
				SAssignNew(MainContentBox, SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding(1.0f, 0.0f)
				[
					SNew(SBorder)
					.BorderImage(StatusBarBackground)
					.VAlign(VAlign_Center)
					.Padding(FMargin(6.0f, 0.0f))
					[
						MakeStatusMessageWidget()
					]
				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.AutoWidth()
				.Padding(1.0f, 0.0f)
				[
					SNew(SBorder)
					.Padding(0.0f)
					.BorderImage(StatusBarBackground)
					[
						MakeStatusBarToolBarWidget()
					]
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.AutoWidth()
				.Padding(1.0f, 0.0f)
				[
					SNew(SBorder)
					.BorderImage(StatusBarBackground)
					.VAlign(VAlign_Center)
					.Padding(FMargin(6.0f, 0.0f))
					[
						MakeProgressBar()
					]
				]
			]
		]
	];

	if (SummonPanelDrawerWidget)
	{
		MainContentBox->AddSlot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Fill)
			.AutoWidth()
			[
				SummonPanelDrawerWidget.ToSharedRef()
			];
	}
}

FStatusBarMessage::FStatusBarMessage(const TAttribute<FText>& InMessageText, const TAttribute<FText>& InHintText, FStatusBarMessageHandle InHandle)
	: MessageText(InMessageText)
	, HintText(InHintText)
	, IconBrush(FAppStyle::Get().GetBrush("StatusBar.HelpIcon"))
	, IconColor(FSlateColor::UseForeground())
	, Handle(InHandle)
{}

FStatusBarMessage::FStatusBarMessage(const TAttribute<FText>& InMessageText, const TAttribute<FText>& InHintText, FStatusBarMessageHandle InHandle,
	const TAttribute<const FSlateBrush*>& InIconBrush, const TAttribute<FSlateColor>& InIconColor)
	: MessageText(InMessageText)
	, HintText(InHintText)
	, IconBrush(InIconBrush)
	, IconColor(InIconColor)
	, Handle(InHandle)
{}

void SStatusBar::PushMessage(FStatusBarMessageHandle InHandle, const TAttribute<FText>& InMessage, const TAttribute<FText>& InHintText)
{
	MessageStack.Emplace(InMessage, InHintText, InHandle);
}

void SStatusBar::PushMessage(FStatusBarMessageHandle InHandle, const TAttribute<FText>& InMessage, const TAttribute<FText>& InHintText,
	const TAttribute<const FSlateBrush*>& InIconBrush, const TAttribute<FSlateColor>& InIconColor)
{
	MessageStack.Emplace(InMessage, InHintText, InHandle, InIconBrush, InIconColor);
}

void SStatusBar::PopMessage(FStatusBarMessageHandle InHandle)
{
	if (InHandle.IsValid() && MessageStack.Num() > 0)
	{
		MessageStack.RemoveAll([InHandle](const FStatusBarMessage& Message)
			{
				return Message.Handle == InHandle;
			});
	}
}

void SStatusBar::ClearAllMessages()
{
	MessageStack.Empty();
}

void SStatusBar::StartProgressNotification(FProgressNotificationHandle InHandle, FText DisplayText, int32 TotalWorkToDo)
{
	if (!FindProgressNotification(InHandle))
	{
		if(TotalWorkToDo > 0)
		{
			ProgressNotifications.Emplace(DisplayText, FPlatformTime::Seconds(), InHandle, TotalWorkToDo);
			UpdateProgressStatus();
		}
	}
}

bool SStatusBar::UpdateProgressNotification(FProgressNotificationHandle InHandle, int32 TotalWorkDone, int32 UpdatedTotalWorkToDo, FText UpdatedDisplayText)
{
	if (FStatusBarProgress* Progress = FindProgressNotification(InHandle))
	{
		if (!UpdatedDisplayText.IsEmpty())
		{
			Progress->DisplayText = UpdatedDisplayText;
		}

		if (UpdatedTotalWorkToDo != 0)
		{
			Progress->TotalWorkToDo = UpdatedTotalWorkToDo;
		}

		Progress->TotalWorkDone = FMath::Clamp(TotalWorkDone, 0, Progress->TotalWorkToDo);

		UpdateProgressStatus();

		return true;
	}

	return false;
}

bool SStatusBar::CancelProgressNotification(FProgressNotificationHandle InHandle)
{
	if (ProgressNotifications.RemoveAll([InHandle](const FStatusBarProgress& Progress){ return Progress.Handle == InHandle;}) != 0)
	{
		UpdateProgressStatus();

		return true;
	}

	return false;
}

EVisibility SStatusBar::GetIconVisibility() const
{
	if (MessageStack.Num() > 0)
	{
		const FStatusBarMessage& MessageData = MessageStack.Top();

		const FText& Message = MessageData.MessageText.Get();
		const FText& HintText = MessageData.HintText.Get();

		return (!Message.IsEmpty() || !HintText.IsEmpty()) ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
	}

	return EVisibility::Collapsed;

}

TSharedPtr<SDockTab> SStatusBar::GetParentTab() const
{
	return ParentTab.Pin();
}

FText SStatusBar::GetStatusBarMessage() const
{
	FText FullMessage;
	if (MessageStack.Num() > 0)
	{
		const FStatusBarMessage& MessageData = MessageStack.Top();

		const FText& Message = MessageData.MessageText.Get();
		const FText& HintText = MessageData.HintText.Get();

		FullMessage = HintText.IsEmpty() ? Message : FText::Format(LOCTEXT("StatusBarMessageFormat", "{0} <StatusBar.Message.InHintText>{1}</>"), Message, HintText);
	}

	return FullMessage;
}

TSharedRef<SWidget> SStatusBar::MakeStatusBarToolBarWidget()
{
	RegisterStatusBarMenu();
	
	FToolMenuContext MenuContext;
	if (bCreateSourceControlSection)
	{
		RegisterSourceControlStatus();
	}

	return UToolMenus::Get()->GenerateWidget(GetToolbarName(), MenuContext);
}

TSharedRef<SWidget> SStatusBar::MakeStatusMessageWidget()
{
	return 
		SNew(SHorizontalBox)
		// Allow the user to mouse over the status bar to read all of it in case it's too long
		.ToolTipText(this, &SStatusBar::GetStatusBarMessage)
		// Default visibility is SelfHitTestInvisible. We want it to be hittestable so we can
		//  show the tooltip when mousing over any part of the bar.
		.Visibility(EVisibility::Visible)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew(SImage)
			.ColorAndOpacity(this, &SStatusBar::GetIconColorAndOpacity)
			.Image(this, &SStatusBar::GetIconBrush)
			.Visibility(this, &SStatusBar::GetIconVisibility)
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(5.0f, 0.0f)
		[
			SNew(SRichTextBlock)
			.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("StatusBar.Message.MessageText"))
			.Text(this, &SStatusBar::GetStatusBarMessage)
			.DecoratorStyleSet(&FAppStyle::Get())
		];
}


TSharedRef<SWidget> SStatusBar::MakeProgressBar()
{
	return 
		SAssignNew(ProgressBar, SStatusBarProgressArea)
		.OnGetProgressMenuContent(this, &SStatusBar::OnGetProgressBarMenuContent);
}

TSharedPtr<SWidget> SStatusBar::MakeSummonPanelDrawer(TArray<UStatusBarSubsystem::FTabIdAndButtonLabel>&& InTabIdAndLabels)
{
	if (InTabIdAndLabels.IsEmpty())
	{
		return {};
	}
	TSharedPtr<SDockTab> ParentTabPtr = ParentTab.Pin();
	TSharedPtr<FTabManager> TabManager = FGlobalTabmanager::Get()->GetTabManagerForMajorTab(ParentTabPtr);

	if (!TabManager || !TabManager->HasPanelDrawer(ParentTabPtr->GetParentWindow()))
	{
		return {};
	}

	TabManager->RegisterOnPanelDrawerStateChanges(
		FTabManager::FOnPanelDrawerStateChanged::FDelegate::CreateSP(this, &SStatusBar::UpdateSummonPanelDrawerArrowRotation)
		);


	// Todo add its own style when more mature and less a proof of concept
	const float StatusBarHalfHeight = FAppStyle::Get().GetFloat("StatusBar.Height") / 2;
	if (!SummonPanelDrawerRoundedBox)
	{
		FVector4 Radius(StatusBarHalfHeight, 0.f, 0.f, StatusBarHalfHeight);
		SummonPanelDrawerRoundedBox = MakeUnique<FSlateRoundedBoxBrush>(FStyleColors::Panel, Radius);
	}

	const bool bOneEntry = InTabIdAndLabels.Num() == 1;

	TSharedPtr<SHorizontalBox> HorizontalBox;
	if (bOneEntry)
	{
		HorizontalBox= SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Fill)
			[
				SNew(SBorder)
				.BorderImage(SummonPanelDrawerRoundedBox.Get())
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("StatusBar.StatusBarButton"))
					.OnClicked(this, &SStatusBar::ToggleTabInPanelDrawer, InTabIdAndLabels[0].Key)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.Padding(FMargin(0.0f, 2.f))
						.VAlign(VAlign_Center)
						.AutoWidth()
						[
							SAssignNew(PanelDrawerArrow, SImage)
								.ColorAndOpacity(FSlateColor::UseForeground())
								
						]
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(FMargin(0.f, 2.f))
						[
							SNew(STextBlock)
								.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
								.Text(InTabIdAndLabels[0].Value)
						]
					]
				]
			];
	}
	else
	{
		// If there is multiple entries
		HorizontalBox = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Fill)
			[
				SNew(SBorder)
				.BorderImage(SummonPanelDrawerRoundedBox.Get())
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("StatusBar.StatusBarButton"))
					.OnClicked(this, &SStatusBar::ClosePanelDrawer)
					[

						SAssignNew(PanelDrawerArrow, SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())

					]
				]
			];

		const FSlateBrush* StatusBarBackground = FAppStyle::Get().GetBrush("Brushes.Panel");

		for (int32 Index = 0; Index < InTabIdAndLabels.Num(); ++Index)
		{
			HorizontalBox->AddSlot()
				.VAlign(VAlign_Fill)
				.AutoWidth()
				.Padding(FMargin(1.f, 0.f, 0.f, 0.f))
				[
					SNew(SBorder)
					.BorderImage(StatusBarBackground)
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
						.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("StatusBar.StatusBarButton"))
						.OnClicked(this, &SStatusBar::ToggleTabInPanelDrawer, InTabIdAndLabels[Index].Key)
						[
							SNew(STextBlock)
							.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
							.Text(InTabIdAndLabels[Index].Value)
						]
					]
				];
		}
	}
	

	UpdateSummonPanelDrawerArrowRotation(*TabManager.Get());

	return HorizontalBox;
}

void SStatusBar::RegisterDrawer(FWidgetDrawerConfig&& Drawer, int32 SlotIndex)
{
	WidgetDrawer->RegisterDrawer(MoveTemp(Drawer), SlotIndex);
}

void SStatusBar::UnregisterDrawer(FName DrawerId)
{
	WidgetDrawer->UnregisterDrawer(DrawerId);
}

void SStatusBar::OpenDrawer(const FName DrawerId)
{
	WidgetDrawer->OpenDrawer(DrawerId);
}

bool SStatusBar::DismissDrawer(const TSharedPtr<SWidget>& NewlyFocusedWidget)
{
	return WidgetDrawer->DismissDrawer(NewlyFocusedWidget);
}

void SStatusBar::CloseDrawerImmediately(FName DrawerId)
{
	WidgetDrawer->CloseDrawerImmediately(DrawerId);
}

bool SStatusBar::IsDrawerOpened(const FName DrawerId) const
{
	return WidgetDrawer->IsDrawerOpened(DrawerId);
}

bool SStatusBar::IsAnyOtherDrawerOpened(const FName DrawerId) const
{
	return WidgetDrawer->IsAnyOtherDrawerOpened(DrawerId);
}

FName SStatusBar::GetStatusBarName() const
{
	return WidgetDrawer->GetDrawerName();
}

FSlateColor SStatusBar::GetIconColorAndOpacity() const
{
	return MessageStack.Num() > 0 ? MessageStack.Top().IconColor.Get() : FSlateColor::UseForeground();
}

const FSlateBrush* SStatusBar::GetIconBrush() const
{
	return MessageStack.Num() > 0 ? MessageStack.Top().IconBrush.Get() : FAppStyle::Get().GetBrush("StatusBar.HelpIcon");
}

FReply SStatusBar::OnDrawerButtonClicked(const FName DrawerId)
{
	if (!IsDrawerOpened(DrawerId))
	{
		OpenDrawer(DrawerId);
	}
	else
	{
		DismissDrawer(nullptr);
	}

	return FReply::Handled();
}

void SStatusBar::RegisterStatusBarMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (ToolMenus->IsMenuRegistered(GetToolbarName()))
	{
		return;
	}

	UToolMenu* ToolBar = ToolMenus->RegisterMenu(GetToolbarName(), NAME_None, EMultiBoxType::SlimHorizontalToolBar);
	ToolBar->StyleName = "StatusBarToolBar";
}

void SStatusBar::RegisterSourceControlStatus()
{
	// Source Control preferences
	FSourceControlMenuHelpers::CheckSourceControlStatus();
	{
		UToolMenu* SourceControlMenu = UToolMenus::Get()->ExtendMenu(GetToolbarName());
		FToolMenuSection& Section = SourceControlMenu->FindOrAddSection("SourceControl");

		Section.AddEntry(
			FToolMenuEntry::InitWidget(
				"SourceControl",
				FSourceControlMenuHelpers::MakeSourceControlStatusWidget(),
				FText::GetEmpty(),
				true,
				false
			));
	}
}

FStatusBarProgress* SStatusBar::FindProgressNotification(FProgressNotificationHandle InHandle)
{
	return ProgressNotifications.FindByPredicate(
		[InHandle](const FStatusBarProgress& Progress)
		{
			return Progress.Handle == InHandle;
		});
}

void SStatusBar::UpdateProgressStatus()
{
	int32 NumIncompleteTasks = 0;

	if (ProgressNotifications.Num())
	{
		int32 TotalWorkToDo = 0;
		int32 CurrentWorkDone = 0;

		bool bShouldAnyProgressBeVisible = false;

		double CurrentTime = FPlatformTime::Seconds();

		const FStatusBarProgress* LastIncompleteTask = &ProgressNotifications.Last();
		for (const FStatusBarProgress& Progress : ProgressNotifications)
		{
			TotalWorkToDo += Progress.TotalWorkToDo;
			CurrentWorkDone += Progress.TotalWorkDone;

			bShouldAnyProgressBeVisible |= ((CurrentTime - Progress.StartTime) >= StatusBarNotificationConstants::NotificationDelay);

			if (Progress.TotalWorkToDo > Progress.TotalWorkDone)
			{
				++NumIncompleteTasks;
				LastIncompleteTask = &Progress;
			}
		}

		// Just assume 100% of the work is done if there is no work to do. The progress bar will dismiss in this case but we want to show 100% while its dismissing
		const float Percent = TotalWorkToDo > 0 ? (float)CurrentWorkDone / TotalWorkToDo : 1.0f;
		FText StatusBarProgressText;
		if (NumIncompleteTasks > 1)
		{
			StatusBarProgressText = FText::Format(LOCTEXT("ProgressBarLabel", "{0} (+{1} more)"), LastIncompleteTask->DisplayText, FText::AsNumber(NumIncompleteTasks - 1));
		}
		else
		{
			StatusBarProgressText = LastIncompleteTask->DisplayText;
		}

		bShouldAnyProgressBeVisible &= (NumIncompleteTasks > 0);
		
		if(bShouldAnyProgressBeVisible)
		{
			OpenProgressBar();

			ProgressBar->SetPercent(Percent);

			ProgressBar->SetProgressText(StatusBarProgressText);
		}
	}

	if (NumIncompleteTasks == 0)
	{
		DismissProgressBar();
	}
}

void SStatusBar::OpenProgressBar()
{
	ProgressBar->OpenProgressBar();
}

void SStatusBar::DismissProgressBar()
{
	ProgressBar->DismissProgressBar();
	ProgressNotifications.Empty();
}

void SStatusBar::UpdateSummonPanelDrawerArrowRotation(const FTabManager& TabManager)
{
	// Todo add its own style when more mature and less a proof of concept
	TSharedPtr<SDockTab> ParentTabPtr = ParentTab.Pin();
	if (TabManager.IsPanelDrawerOpen(ParentTabPtr->GetParentWindow()))
	{
		PanelDrawerArrow->SetImage(FAppStyle::Get().GetBrush(TEXT("Icons.ChevronRight")));
	}
	else
	{
		PanelDrawerArrow->SetImage(FAppStyle::Get().GetBrush(TEXT("Icons.ChevronLeft")));
	}
}

FReply SStatusBar::ToggleTabInPanelDrawer(FName TabId)
{
	TSharedPtr<SDockTab> ParentTabPtr = ParentTab.Pin();
	if (TSharedPtr<FTabManager> TabManager = FGlobalTabmanager::Get()->GetTabManagerForMajorTab(ParentTabPtr))
	{
		constexpr bool bForceInPanelDrawer = true;
		TabManager->TryToggleTabInPanelDrawer(TabId, ParentTabPtr->GetParentWindow(), bForceInPanelDrawer);
	}

	return FReply::Handled();
}

FReply SStatusBar::ClosePanelDrawer()
{
	TSharedPtr<SDockTab> ParentTabPtr = ParentTab.Pin();
	if (TSharedPtr<FTabManager> TabManager = FGlobalTabmanager::Get()->GetTabManagerForMajorTab(ParentTabPtr))
	{
		TabManager->ClosePanelDrawer(ParentTabPtr->GetParentWindow());
	}

	return FReply::Handled();
}

TSharedRef<SWidget> SStatusBar::OnGetProgressBarMenuContent()
{
	FMenuBuilder ProgressBarMenu(false, nullptr);

	const float StatusBarHeight = FAppStyle::Get().GetFloat("StatusBar.Height");

	for (int32 ProgressIndex = 0; ProgressIndex < ProgressNotifications.Num(); ++ProgressIndex)
	{
		FStatusBarProgress& Progress = ProgressNotifications[ProgressIndex];

		FProgressNotificationHandle Handle = Progress.Handle;

		const bool bLastProgressBar = (ProgressIndex + 1 == ProgressNotifications.Num());

		TSharedRef<SWidget> MenuWidget =
			SNew(SBox)
			.Padding(FMargin(8.0f, ProgressIndex == 0 ? 0.0f : 4.0f, 8.0f, bLastProgressBar ? 0.0f : 8.0f))
			[
				SNew(SStatusBarProgressWidget)
				.StatusBarProgress_Lambda([this, Handle]() { return FindProgressNotification(Handle); })
			];


		ProgressBarMenu.AddWidget(MenuWidget, FText::GetEmpty(), false, false);

		if (!bLastProgressBar)
		{
			ProgressBarMenu.AddWidget(SNew(SSeparator).Thickness(1.0f), FText::GetEmpty(), false, false);
		}
	}

	return
		SNew(SBox)
		.WidthOverride((float)ProgressBar->GetDesiredSize().X-8.0f)
		[
			ProgressBarMenu.MakeWidget()
		];
}

FName SStatusBar::GetToolbarName() const
{
	return FName(*(WidgetDrawer->GetSerializableName() + ".ToolBar"));
}

#undef LOCTEXT_NAMESPACE