// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "StatusBarSubsystem.h"
#include "Styling/SlateBrush.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

DECLARE_DELEGATE_OneParam(FOnStatusBarDrawerTargetHeightChanged, float);

class FTabManager;
class SDockTab;
class SDrawerOverlay;
class SHorizontalBox;
class SMultiLineEditableTextBox;
class SStatusBarProgressArea;
class SStatusBarProgressWidget;
class SWidget;
class SWidgetDrawer;
class SWindow;
struct FSlateRoundedBoxBrush;
struct FSlateColor;
struct FSlateBrush;

namespace StatusBarDrawerIds
{
	const FName ContentBrowser("ContentBrowser");
	const FName OutputLog("OutputLog");
}

/** Data payload for messages in the status bar */
struct FStatusBarMessage
{
	FStatusBarMessage(const TAttribute<FText>& InMessageText, const TAttribute<FText>& InHintText, FStatusBarMessageHandle InHandle);

	FStatusBarMessage(const TAttribute<FText>& InMessageText, const TAttribute<FText>& InHintText, FStatusBarMessageHandle InHandle,
		const TAttribute<const FSlateBrush*>& InIconBrush, const TAttribute<FSlateColor>& InIconColor);

	TAttribute<FText> MessageText;
	TAttribute<FText> HintText;
	TAttribute<const FSlateBrush*> IconBrush;
	TAttribute<FSlateColor> IconColor;
	FStatusBarMessageHandle Handle;
};

/** Data payload for progress bars in the status bar */
struct FStatusBarProgress
{
	FStatusBarProgress(FText InDisplayText, double InStartTime, FProgressNotificationHandle InHandle, int32 InTotalWorkToDo)
		: DisplayText(InDisplayText)
		, StartTime(InStartTime)
		, TotalWorkToDo(InTotalWorkToDo)
		, TotalWorkDone(0)
		, Handle(InHandle)
	{}

	FText DisplayText;
	double StartTime;
	int32 TotalWorkToDo;
	int32 TotalWorkDone;
	FProgressNotificationHandle Handle;
};

class SStatusBar : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SStatusBar)	
	{}
		SLATE_ARGUMENT_DEFAULT(bool, CreateSourceControlSection) = true;
	SLATE_END_ARGS()

public:
	~SStatusBar();

	/** SWidget interface */
	virtual bool SupportsKeyboardFocus() const { return false; }
	void Construct(const FArguments& InArgs, FName InStatusBarName, const TSharedRef<SDockTab> InParentTab, TArray<UStatusBarSubsystem::FTabIdAndButtonLabel>&& PanelDrawerSummons);

	/**
	 * Pushes a new status bar message
	 *
	 * @param InHandle	A handle to the message for clearing it later
	 * @param InMessage	The message to display
	 * @param InHintText	Optional hint text message.  This message will be highlighted to make it stand out
	 */
	void PushMessage(FStatusBarMessageHandle InHandle, const TAttribute<FText>& InMessage, const TAttribute<FText>& InHintText);

	/**
	 * Pushes a new status bar message with a custom icon and color
	 *
	 * @param InHandle A handle to the message for clearing it later
	 * @param InMessage The message to display
	 * @param InHintText Optional hint text. Highlighted to stand out
	 * @param InIconBrush Custom icon brush. Pass nullptr to use the default help icon
	 * @param InIconColor Color/opacity for the icon, supports dynamic attributes for pulsing effects
	 */
	void PushMessage(FStatusBarMessageHandle InHandle, const TAttribute<FText>& InMessage, const TAttribute<FText>& InHintText,
		const TAttribute<const FSlateBrush*>& InIconBrush, const TAttribute<FSlateColor>& InIconColor);

	/**
	 * Removes a message from the status bar.  When messages are removed the previous message on the stack (if any) is displayed
	 *
	 * @param InHandle	Handle to the status bar message to remove
	 */
	void PopMessage(FStatusBarMessageHandle InHandle);

	/**
	 * Removes all messages from the status bar
	 */
	void ClearAllMessages();

	/**
	 * Called when a progress notification begins
	 * @param Handle			Handle to the notification
	 * @param DisplayText		Display text used to describe the type of work to the user
	 * @param TotalWorkToDo		Arbitrary number of work units to perform.
	 */
	void StartProgressNotification(FProgressNotificationHandle InHandle, FText DisplayText, int32 TotalWorkToDo);

	/**
	 * Called when a notification should be updated.
	 * @param InHandle				Handle to the notification that was previously created with StartProgressNotification
	 * @param TotalWorkDone			The total number of work units done for the notification.
	 * @param UpdatedTotalWorkToDo	UpdatedTotalWorkToDo. This value will be 0 if the total work did not change
	 * @param UpdatedDisplayText	Updated display text of the notification. This value will be empty if the text did not change
	 */
	bool UpdateProgressNotification(FProgressNotificationHandle InHandle, int32 TotalWorkDone, int32 UpdatedTotalWorkToDo, FText UpdatedDisplayText);

	/**
	 * Called when a notification should be canceled
	 */
	bool CancelProgressNotification(FProgressNotificationHandle InHandle);

	/**
	 * Gets the owning major tab that this status bar is in
	 */
	TSharedPtr<SDockTab> GetParentTab() const;

	/**
	 * Registers a new drawer with this status bar. Registering will add a button to open and close the drawer
	 */
	void RegisterDrawer(FWidgetDrawerConfig&& Drawer, int32 SlotIndex = INDEX_NONE);

	/**
	 * Unregisters drawer with the given DrawerId.
	 * This will dismiss the drawer immediately if its opened, and destroy the status bar widget that opens/closes the drawer
	 *
	 * @param DrawerId name of the registered drawer to unregister 
	 */
	void UnregisterDrawer(FName DrawerId);

	/**
	 * Opens a drawer
	 * 
	 * @param DrawerId Name of the registered drawer to open 
	 */
	void OpenDrawer(const FName DrawerId);

	/**
	 * Dismisses an open drawer with an animation.  The drawer contents are removed once the animation is complete
	 * 
	 * @param NewlyFocusedWidget Optional widget to focus 
	 * @return true if any open drawer was dismissed
	 */
	bool DismissDrawer(const TSharedPtr<SWidget>& NewlyFocusedWidget);

	/**
	 * Closes a drawer immediately with no closing animation. Needed for when UI is shutting down or to prevent conflicts where a drawer is open in two places at once. 
	 * @param DrawerId The name of the drawer to close. If no name is specified all drawers are closed immediately
	 */
	void CloseDrawerImmediately(FName DrawerId = NAME_None);

	/** Is a specific registered drawer currently open */
	bool IsDrawerOpened(const FName DrawerId) const;

	/** Is any drawer other than the one specified by DrawerId opened */
	bool IsAnyOtherDrawerOpened(const FName DrawerId) const;

	/** Get unique name of this status bar */
	FName GetStatusBarName() const;

private:

	FSlateColor GetIconColorAndOpacity() const;
	const FSlateBrush* GetIconBrush() const;
	EVisibility GetIconVisibility() const;
	FText GetStatusBarMessage() const;

	TSharedRef<SWidget> MakeStatusBarToolBarWidget();
	TSharedRef<SWidget> MakeStatusMessageWidget();
	TSharedRef<SWidget> MakeProgressBar();
	TSharedPtr<SWidget> MakeSummonPanelDrawer(TArray<UStatusBarSubsystem::FTabIdAndButtonLabel>&& InTabIdAndLabels);

	FReply OnDrawerButtonClicked(const FName DrawerId);

	void RegisterStatusBarMenu();
	void RegisterSourceControlStatus();

	FStatusBarProgress* FindProgressNotification(FProgressNotificationHandle InHandle);

	void UpdateProgressStatus();
	void OpenProgressBar();
	void DismissProgressBar();
	
	void UpdateSummonPanelDrawerArrowRotation(const FTabManager& TabManager);
	FReply ToggleTabInPanelDrawer(FName TabId);
	FReply ClosePanelDrawer();

	TSharedRef<SWidget> OnGetProgressBarMenuContent();

	FName GetToolbarName() const;
	
private:
	
	TArray<FStatusBarMessage> MessageStack;
	TArray<FStatusBarProgress> ProgressNotifications;

	TWeakPtr<SDockTab> ParentTab;
	TSharedPtr<SWidgetDrawer> WidgetDrawer;
	TSharedPtr<SStatusBarProgressArea> ProgressBar;

	TSharedPtr<SImage> PanelDrawerArrow;
	static TUniquePtr<FSlateRoundedBoxBrush> SummonPanelDrawerRoundedBox;

	const FSlateBrush* UpArrow = nullptr;
	const FSlateBrush* DownArrow = nullptr;

	/** Whether or not to create the source control section. */
	bool bCreateSourceControlSection = true;
};

