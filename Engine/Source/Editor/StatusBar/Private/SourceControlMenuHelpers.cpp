// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceControlMenuHelpers.h"
#include "SourceControlOperations.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "ISourceControlWindowsModule.h"
#include "SourceControlProviders.h"
#include "SourceControlWindows.h"
#include "UnsavedAssetsTrackerModule.h"
#include "FileHelpers.h"
#include "Logging/MessageLog.h"
#include "ToolMenuContext.h"
#include "ToolMenus.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Images/SLayeredImage.h"
#include "PackageTools.h"
#include "Editor.h"
#include "EditorModeManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "RevisionControlStyle/RevisionControlStyle.h"
#include "Styling/StyleColors.h"
#include "HAL/IConsoleManager.h"
#include "SSourceControlControls.h"
#include "LevelEditor.h"

#define LOCTEXT_NAMESPACE "SourceControlCommands"

TSharedRef<FUICommandList> FSourceControlCommands::ActionList(new FUICommandList());

FSourceControlCommands::FSourceControlCommands() 
	: TCommands<FSourceControlCommands>
(
	"SourceControl",
	NSLOCTEXT("Contexts", "SourceControl", "Revision Control"),
	"LevelEditor",
	FAppStyle::GetAppStyleSetName()
)
{}

/**
 * Initialize commands
 */
void FSourceControlCommands::RegisterCommands()
{
	UI_COMMAND(ConnectToSourceControl, "Connect to Revision Control...", "Connect to a revision control system for tracking changes to your content and levels.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ChangeSourceControlSettings, "Change Revision Control Settings...", "Opens a dialog to change revision control settings.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ViewChangelists, "View Changes", "Opens a dialog displaying current changes.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SubmitContent, "Submit Content", "Opens a dialog with check in options for content and levels.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ViewSnapshotHistory, "Open Branch History", "See all the changes that have been made to this branch over time.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ViewBranchExplorer, "Open Branch Explorer (Beta)", "See all the branches which this project uses.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Alt, EKeys::B));
	UI_COMMAND(CheckOutModifiedFiles, "Check Out Modified Files", "Opens a dialog to check out any assets which have been modified.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RevertAll, "Revert All Files", "Opens a dialog to revert any assets which have been modified.", EUserInterfaceActionType::Button, FInputChord());

	ActionList->MapAction(
		ConnectToSourceControl,
		FExecuteAction::CreateStatic(&FSourceControlCommands::ConnectToSourceControl_Clicked)
	);

	ActionList->MapAction(
		ChangeSourceControlSettings,
		FExecuteAction::CreateStatic(&FSourceControlCommands::ConnectToSourceControl_Clicked)
	);

	ActionList->MapAction(
		SubmitContent,
		FExecuteAction::CreateStatic(&FSourceControlCommands::SubmitContent_Clicked),
		FCanExecuteAction::CreateStatic(&FSourceControlCommands::SubmitContent_CanExecute),
		FIsActionChecked::CreateLambda([]() { return false; }),
		FIsActionButtonVisible::CreateStatic(&FSourceControlCommands::SubmitContent_IsVisible)
	);


	ActionList->MapAction(
		ViewChangelists,
		FExecuteAction::CreateStatic(&FSourceControlCommands::ViewChangelists_Clicked),
		FCanExecuteAction::CreateStatic(&FSourceControlCommands::ViewChangelists_CanExecute),
		FIsActionChecked::CreateLambda([]() { return false; }),
		FIsActionButtonVisible::CreateStatic(&FSourceControlCommands::ViewChangelists_IsVisible)
	);

	ActionList->MapAction(
		ViewSnapshotHistory,
		FExecuteAction::CreateStatic(&FSourceControlCommands::ViewSnapshotHistory_Clicked),
		FCanExecuteAction::CreateStatic(&FSourceControlCommands::ViewSnapshotHistory_CanExecute),
		FIsActionChecked::CreateLambda([]() { return false; }),
		FIsActionButtonVisible::CreateStatic(&FSourceControlCommands::ViewSnapshotHistory_IsVisible)
	);

	ActionList->MapAction(
		ViewBranchExplorer,
		FExecuteAction::CreateStatic(&FSourceControlCommands::ViewBranchExplorer_Clicked),
		FCanExecuteAction::CreateStatic(&FSourceControlCommands::ViewBranchExplorer_CanExecute),
		FIsActionChecked::CreateLambda([]() { return false; }),
		FIsActionButtonVisible::CreateStatic(&FSourceControlCommands::ViewBranchExplorer_IsVisible)
	);

	// Add to global command list to allow key shortcut anywhere in Editor.
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<FUICommandList> CommandList = LevelEditorModule.GetGlobalLevelEditorActions();
	if (CommandList.IsValid())
	{
		CommandList->MapAction(
			ViewBranchExplorer,
			FExecuteAction::CreateStatic(&FSourceControlCommands::ViewBranchExplorer_Clicked),
			FCanExecuteAction::CreateStatic(&FSourceControlCommands::ViewBranchExplorer_CanExecute)
		);
	}

	ActionList->MapAction(
		CheckOutModifiedFiles,
		FExecuteAction::CreateStatic(&FSourceControlCommands::CheckOutModifiedFiles_Clicked),
		FCanExecuteAction::CreateStatic(&FSourceControlCommands::CheckOutModifiedFiles_CanExecute),
		FIsActionChecked::CreateLambda([]() { return false; }),
		FIsActionButtonVisible::CreateStatic(&FSourceControlCommands::CheckOutModifiedFiles_IsVisible)
	);

	ActionList->MapAction(
		RevertAll,
		FExecuteAction::CreateStatic(&FSourceControlCommands::RevertAllModifiedFiles_Clicked),
		FCanExecuteAction::CreateStatic(&FSourceControlCommands::RevertAllModifiedFiles_CanExecute)
	);
}

void FSourceControlCommands::ConnectToSourceControl_Clicked()
{
	// Show login window regardless of current status - its useful as a shortcut to change settings.
	ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
	SourceControlModule.ShowLoginDialog(FSourceControlLoginClosed(), ELoginWindowMode::Modeless, EOnLoginWindowStartup::PreserveProvider);
}

bool FSourceControlCommands::ViewChangelists_CanExecute()
{
	return ISourceControlWindowsModule::Get().CanShowChangelistsTab();
}

bool FSourceControlCommands::ViewChangelists_IsVisible()
{
	return ISourceControlModule::Get().GetProvider().UsesChangelists() || ISourceControlModule::Get().GetProvider().UsesUncontrolledChangelists();
}

bool FSourceControlCommands::SubmitContent_CanExecute()
{
	return FSourceControlWindows::CanChoosePackagesToCheckIn();
}

bool FSourceControlCommands::SubmitContent_IsVisible()
{
	// Access to the legacy 'Submit Files' dialog. Git / Subversion require this.

	if (ISourceControlModule::Get().GetProvider().UsesChangelists())
	{
		return false;
	}

	if (ISourceControlModule::Get().GetProvider().GetName() == SourceControlProviders::GetUrcProviderName())
	{
		return false;
	}

	return ISourceControlModule::Get().IsEnabled();
}

bool FSourceControlCommands::ViewSnapshotHistory_CanExecute()
{
	return ISourceControlWindowsModule::Get().CanShowSnapshotHistoryTab();
}

bool FSourceControlCommands::ViewSnapshotHistory_IsVisible()
{
	return ISourceControlModule::Get().GetProvider().GetName() == SourceControlProviders::GetUrcProviderName();
}

bool FSourceControlCommands::ViewBranchExplorer_CanExecute()
{
	return ISourceControlWindowsModule::Get().CanShowBranchExplorerTab();
}

bool FSourceControlCommands::ViewBranchExplorer_IsVisible()
{
	if (IConsoleManager::Get().FindConsoleObject(TEXT("UnrealRevisionControl.FocusBranchExplorer")) == nullptr)
	{
		return false;
	}

	return ISourceControlModule::Get().GetProvider().GetName() == SourceControlProviders::GetUrcProviderName();
}

void FSourceControlCommands::ViewChangelists_Clicked()
{
	ISourceControlWindowsModule::Get().ShowChangelistsTab();
}

void FSourceControlCommands::SubmitContent_Clicked()
{
	FSourceControlWindows::ChoosePackagesToCheckIn();
}

void FSourceControlCommands::ViewSnapshotHistory_Clicked()
{
	ISourceControlWindowsModule::Get().ShowSnapshotHistoryTab();
}

void FSourceControlCommands::ViewBranchExplorer_Clicked()
{
	ISourceControlWindowsModule::Get().ShowBranchExplorerTab();
}

bool FSourceControlCommands::CheckOutModifiedFiles_CanExecute()
{
	ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
	if (ISourceControlModule::Get().IsEnabled() &&
		ISourceControlModule::Get().GetProvider().IsAvailable())
	{
		return FEditorFileUtils::HasDirtyPackages();
	}

	return false;
}

bool FSourceControlCommands::CheckOutModifiedFiles_IsVisible()
{
	return ISourceControlModule::Get().GetProvider().UsesCheckout();
}

void FSourceControlCommands::CheckOutModifiedFiles_Clicked()
{
	TArray<UPackage*> PackagesToSave;
	FEditorFileUtils::GetDirtyPackages(PackagesToSave);

	FEditorFileUtils::FPromptForCheckoutAndSaveParams SaveParams;
	SaveParams.bCheckDirty = true;
	SaveParams.bPromptToSave = false;
	SaveParams.bIsExplicitSave = true;

	FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, SaveParams);
}

bool FSourceControlCommands::RevertAllModifiedFiles_CanExecute()
{
	// If CHECK IN CHANGES button is active, the REVERT ALL option should be active.

	if (FUnsavedAssetsTrackerModule::Get().GetUnsavedAssetNum() > 0)
	{
		return true;
	}

	if (FSourceControlWindows::CanChoosePackagesToCheckIn())
	{
		return true;
	}

	return false;
}

void FSourceControlCommands::RevertAllModifiedFiles_Clicked()
{
	FText Message = LOCTEXT("RevertAllModifiedFiles", "Are you sure you want to revert all local changes? By proceeding, your local changes will be discarded, and the state of your last synced revision will be restored.");
	FText Title = LOCTEXT("RevertAllModifiedFiles_Title", "Revert all local changes");
	if (FMessageDialog::Open(EAppMsgType::YesNo, EAppReturnType::No, Message, Title) == EAppReturnType::Yes)
	{
		const bool bPromptUserToSave = false;
		const bool bSaveMapPackages = true;
		const bool bSaveContentPackages = true;
		const bool bFastSave = false;
		const bool bNotifyNoPackagesSaved = false;
		const bool bCanBeDeclined = false;
		FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages, bFastSave, bNotifyNoPackagesSaved, bCanBeDeclined);

		FSourceControlWindows::RevertAllChangesAndReloadWorld();
	}
}

FSourceControlMenuHelpers::EQueryState FSourceControlMenuHelpers::QueryState = FSourceControlMenuHelpers::EQueryState::NotQueried;

void FSourceControlMenuHelpers::CheckSourceControlStatus()
{
	ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
	if (SourceControlModule.IsEnabled())
	{
		TSharedRef<FConnect> Operation = ISourceControlOperation::Create<FConnect>();
		// Disable logging of info messages to prevent dumping 100+ rows of client info to the log every time an asset editor is opened.
		Operation->SetEnableInfoLogging(false);
		SourceControlModule.GetProvider().Execute(
			Operation,
			EConcurrency::Asynchronous,
			FSourceControlOperationComplete::CreateStatic(&FSourceControlMenuHelpers::OnSourceControlOperationComplete)
		);
		QueryState = EQueryState::Querying;
	}
}

void FSourceControlMenuHelpers::OnSourceControlOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	QueryState = EQueryState::Queried;
}

TSharedRef<SWidget> FSourceControlMenuHelpers::GenerateSourceControlMenuContent()
{
	// NOTE: This menu should be in sync with MainMenu's SourceControl menu.

	UToolMenu* SourceControlMenu = UToolMenus::Get()->RegisterMenu("StatusBar.ToolBar.SourceControl", NAME_None, EMultiBoxType::Menu, false);

	FToolMenuSection& Section = SourceControlMenu->AddSection("SourceControlActions", LOCTEXT("SourceControlMenuHeadingActions", "Actions"));

	Section.AddMenuEntry(
		FSourceControlCommands::Get().ViewChangelists,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.ChangelistsTab")
	);

	Section.AddMenuEntry(
		FSourceControlCommands::Get().SubmitContent,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.Submit")
	);

	Section.AddMenuEntry(
		FSourceControlCommands::Get().ViewSnapshotHistory,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.Rewind")
	);

	Section.AddMenuEntry(
		FSourceControlCommands::Get().ViewBranchExplorer,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.BranchExplorerTab")
	);

	Section.AddMenuEntry(
		FSourceControlCommands::Get().CheckOutModifiedFiles,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.CheckOut")
	);

	Section.AddDynamicEntry("ConnectToSourceControl", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
		if (ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable())
		{
			InSection.AddMenuEntry(
				FSourceControlCommands::Get().ChangeSourceControlSettings,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.ChangeSettings")
			);
		}
		else
		{
			InSection.AddMenuEntry(
				FSourceControlCommands::Get().ConnectToSourceControl,
				TAttribute<FText>(),
				TAttribute<FText>(),
				FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.Connect")
			);
		}
	}));

	return UToolMenus::Get()->GenerateWidget("StatusBar.ToolBar.SourceControl", FToolMenuContext(FSourceControlCommands::ActionList));
}

TSharedRef<SWidget> FSourceControlMenuHelpers::GenerateCheckInComboButtonContent()
{
	UToolMenu* SourceControlMenu = UToolMenus::Get()->RegisterMenu("StatusBar.ToolBar.SourceControl.CheckInCombo", NAME_None, EMultiBoxType::Menu, false);

	FToolMenuSection& Section = SourceControlMenu->AddSection("SourceControlComboActions", LOCTEXT("SourceControlComboMenuHeadingActions", "Actions"));

	Section.AddMenuEntry(
		FSourceControlCommands::Get().RevertAll,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FRevisionControlStyleManager::Get().GetStyleSetName(), "RevisionControl.Actions.Revert")
	);

	return UToolMenus::Get()->GenerateWidget("StatusBar.ToolBar.SourceControl.CheckInCombo", FToolMenuContext(FSourceControlCommands::ActionList));
}

FText FSourceControlMenuHelpers::GetSourceControlStatusText()
{
	if (QueryState == EQueryState::Querying)
	{
		return LOCTEXT("SourceControlStatus_Querying", "Contacting Server....");
	}
	else
	{
		ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
		if (SourceControlModule.IsEnabled())
		{
			if (!SourceControlModule.GetProvider().IsAvailable())
			{
				return LOCTEXT("SourceControlStatus_Error_ServerUnavailable", "Server Unavailable");
			}
			else
			{
				return LOCTEXT("SourceControlStatus_Available", "Revision Control");
			}
		}
		else
		{
			return LOCTEXT("SourceControlStatus_Error_Off", "Revision Control"); // Relies on the icon on the status bar widget to know if the source control is on or off.
		}
	}
}
FText FSourceControlMenuHelpers::GetSourceControlTooltip()
{
	if (QueryState == EQueryState::Querying)
	{
		return LOCTEXT("SourceControlUnknown", "Revision control status is unknown");
	}
	else
	{
		return ISourceControlModule::Get().GetProvider().GetStatusText();
	}
}

const FSlateBrush* FSourceControlMenuHelpers::GetSourceControlIconBadge()
{
	if (QueryState == EQueryState::Querying)
	{
		return nullptr;
	}
	else
	{
		ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
		if (SourceControlModule.IsEnabled())
		{
			if (!SourceControlModule.GetProvider().IsAvailable())
			{
				static const FSlateBrush* ErrorBrush = FRevisionControlStyleManager::Get().GetBrush("RevisionControl.Icon.WarningBadge");
				return ErrorBrush;
			}
			else
			{
				static const FSlateBrush* OnBrush = FRevisionControlStyleManager::Get().GetBrush("RevisionControl.Icon.ConnectedBadge");
				return OnBrush;
			}
		}
		else
		{
			static const FSlateBrush* OffBrush = nullptr;
			return OffBrush;
		}
	}
}

TSharedRef<SWidget> FSourceControlMenuHelpers::MakeSourceControlStatusWidget()
{
	TSharedRef<SLayeredImage> SourceControlIcon =
		SNew(SLayeredImage)
		.ColorAndOpacity(FSlateColor::UseForeground())
		.Image(FRevisionControlStyleManager::Get().GetBrush("RevisionControl.Icon"));

	SourceControlIcon->AddLayer(TAttribute<const FSlateBrush*>::CreateStatic(&FSourceControlMenuHelpers::GetSourceControlIconBadge));
	
	return
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			// NOTE: The unsaved can have its own menu extension in the status bar to decouple it from source control, but putting all the buttons in the right order
			//       on the status bar is not deterministic, you need to know the name of the menu that is before or after and the menu is dynamic. Having it here
			//       ensure its position with respect to the source control button.
			FUnsavedAssetsTrackerModule::Get().MakeUnsavedAssetsStatusBarWidget()
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.0f, -5.0f) // Intentional negative padding to make the separator cover the whole status bar vertically
		[
			SNew(SSeparator)
			.Thickness(2.0f)
			.Orientation(EOrientation::Orient_Vertical)
		]
		+SHorizontalBox::Slot()
		.Padding(0.f)
		[
			SNew(SSourceControlControls)
			.OnGenerateKebabMenu_Static(&FSourceControlMenuHelpers::GenerateCheckInComboButtonContent)
		]
		+ SHorizontalBox::Slot() // Source Control Menu
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SComboButton)
			.ContentPadding(FMargin(6.0f, 0.0f))
			.ToolTipText_Static(&FSourceControlMenuHelpers::GetSourceControlTooltip)
			.MenuPlacement(MenuPlacement_AboveAnchor)
			.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SourceControlIcon
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(5, 0, 0, 0))
				[
					SNew(STextBlock)
					.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
					.Text_Static(&FSourceControlMenuHelpers::GetSourceControlStatusText)
				]
			]
			.OnGetMenuContent(FOnGetContent::CreateStatic(&FSourceControlMenuHelpers::GenerateSourceControlMenuContent))
		];
}

#undef LOCTEXT_NAMESPACE
