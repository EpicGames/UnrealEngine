// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "ISourceControlOperation.h"
#include "ISourceControlProvider.h"

class FUICommandList;
class FReply;
class SWidget;
class SSourceControlControls;

class FSourceControlCommands : public TCommands<FSourceControlCommands>
{
public:
	FSourceControlCommands();

	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;

private:

	static void ConnectToSourceControl_Clicked();
	static bool ViewChangelists_CanExecute();
	static bool ViewChangelists_IsVisible();
	static void ViewChangelists_Clicked();
	static bool SubmitContent_CanExecute();
	static bool SubmitContent_IsVisible();
	static void SubmitContent_Clicked();
	static bool ViewSnapshotHistory_CanExecute();
	static bool ViewSnapshotHistory_IsVisible();
	static void ViewSnapshotHistory_Clicked();
	static bool ViewBranchExplorer_CanExecute();
	static bool ViewBranchExplorer_IsVisible();
	static void ViewBranchExplorer_Clicked();
	static bool CheckOutModifiedFiles_CanExecute();
	static bool CheckOutModifiedFiles_IsVisible();
	static void CheckOutModifiedFiles_Clicked();
	static bool RevertAllModifiedFiles_CanExecute();
	static void RevertAllModifiedFiles_Clicked();

public:
	/**
	 * Source Control Commands
	 */
	TSharedPtr< FUICommandInfo > ConnectToSourceControl;
	TSharedPtr< FUICommandInfo > ChangeSourceControlSettings;
	TSharedPtr< FUICommandInfo > ViewChangelists;
	TSharedPtr< FUICommandInfo > SubmitContent;
	TSharedPtr< FUICommandInfo > ViewSnapshotHistory;
	TSharedPtr< FUICommandInfo > ViewBranchExplorer;
	TSharedPtr< FUICommandInfo > CheckOutModifiedFiles;
	TSharedPtr< FUICommandInfo > RevertAll;

	static TSharedRef<FUICommandList> ActionList;
};

class FSourceControlMenuHelpers
{
	friend class FSourceControlCommands;

private:
	FSourceControlMenuHelpers() {};

private:
	enum EQueryState
	{
		NotQueried,
		Querying,
		Queried,
	};

	static EQueryState QueryState;

public:
	static void CheckSourceControlStatus();
	static TSharedRef<SWidget> MakeSourceControlStatusWidget();

private:
	static void OnSourceControlOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);
	static TSharedRef<SWidget> GenerateSourceControlMenuContent();
	static TSharedRef<SWidget> GenerateCheckInComboButtonContent();
	static FText GetSourceControlStatusText();
	static FText GetSourceControlTooltip();
	static const FSlateBrush* GetSourceControlIconBadge();

private:
	/** Delegate handles */
	FDelegateHandle SourceControlProviderChangedHandle;
	FDelegateHandle SourceControlStateChangedHandle;

};

