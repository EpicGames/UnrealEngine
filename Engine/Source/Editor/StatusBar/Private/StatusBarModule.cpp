// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatusBarModule.h"


class FStatusBarModule : public IStatusBarModule
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE( FStatusBarModule, StatusBar )



void FStatusBarModule::StartupModule()
{

}


void FStatusBarModule::ShutdownModule()
{

}



