// Copyright Epic Games, Inc. All Rights Reserved.


#include "StatusBarSubsystem.h"
#include "ToolMenus.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Application/SlateApplication.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "SourceControlMenuHelpers.h"
#include "SStatusBar.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "Editor.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Interfaces/IMainFrameModule.h"
#include "Widgets/Notifications/SNotificationBackground.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Misc/ConfigCacheIni.h"
#include "OutputLogModule.h"
#include "WidgetDrawerConfig.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "OutputLogSettings.h"
#include "SOneTimeIndustryQuery.h"
#include "Types/SlateAttributeMetaData.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StatusBarSubsystem)

#define LOCTEXT_NAMESPACE "StatusBar"

DEFINE_LOG_CATEGORY_STATIC(LogStatusBar, Log, All);

int32 UStatusBarSubsystem::MessageHandleCounter = 0;

namespace UE
{
	namespace StatusBarSubsystem
	{
		namespace Private
		{
			TSharedPtr<SWindow> FindParentWindow()
			{
				TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().GetActiveTopLevelWindow();
				if (!ParentWindow.IsValid())
				{
					if (TSharedPtr<SDockTab> ActiveTab = FGlobalTabmanager::Get()->GetActiveTab())
					{
						if (TSharedPtr<FTabManager> ActiveTabManager = ActiveTab->GetTabManagerPtr())
						{
							if (TSharedPtr<SDockTab> ActiveMajorTab = FGlobalTabmanager::Get()->GetMajorTabForTabManager(ActiveTabManager.ToSharedRef()))
							{
								ParentWindow = ActiveMajorTab->GetParentWindow();
							}
						}
					}
				}

				return ParentWindow;
			}

			TSharedPtr<SWindow> FindParentWindowBehindNotification()
			{
				TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().GetActiveTopLevelWindow();
				if (!ParentWindow.IsValid())
				{
					if (TSharedPtr<SDockTab> ActiveTab = FGlobalTabmanager::Get()->GetActiveTab())
					{
						if (TSharedPtr<FTabManager> ActiveTabManager = ActiveTab->GetTabManagerPtr())
						{
							if (TSharedPtr<SDockTab> ActiveMajorTab = FGlobalTabmanager::Get()->GetMajorTabForTabManager(ActiveTabManager.ToSharedRef()))
							{
								ParentWindow = ActiveMajorTab->GetParentWindow();
							}
						}
					}
				}
				else if (ParentWindow->GetType() == EWindowType::Notification)
				{
					// Get the parent window directly behind the notification. 
					ParentWindow = FSlateApplication::Get().GetActiveTopLevelRegularWindow();
				}

				return ParentWindow;
			}

			TArray<UStatusBarSubsystem::FTabIdAndButtonLabel> CreatePanelDrawerSummonsData(const TSharedRef<SDockTab>& InParentDockTab, UStatusBarSubsystem::FRegisterPanelDrawerSummonDelegate& InPanelDrawerSummonsDelegate)
			{
				// Not great but that will do it as quick toggle to turn off the feature in 5.7 if needed
				if (TConsoleVariableData<bool>* EnablePanelDrawer = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("EnablePanelDrawer")))
				{
					if (!EnablePanelDrawer->GetValueOnGameThread())
					{
						return {};
					}
				}

				TSharedPtr<FTabManager> TabManager = InParentDockTab->GetTabManagerPtr();
				if (InParentDockTab->GetTabRole() == ETabRole::MajorTab)
				{
					TabManager = FGlobalTabmanager::Get()->GetTabManagerForMajorTab(InParentDockTab);
				}

				if (!TabManager || !TabManager->HasPanelDrawer(InParentDockTab->GetParentWindow()))
				{
					return {};
				}

				TArray<UStatusBarSubsystem::FTabIdAndButtonLabel> PanelDrawerSummons;
				InPanelDrawerSummonsDelegate.Broadcast(PanelDrawerSummons, InParentDockTab);
				return PanelDrawerSummons;
			}
		}
	}
}

class SNewUserTipNotification : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SNewUserTipNotification)
	{}
	SLATE_END_ARGS()

	SNewUserTipNotification()
		: NewBadgeBrush(FStyleColors::Success)
		, KeybindBackgroundBrush(FLinearColor::Transparent, 6.0f, FStyleColors::ForegroundHover, 1.5f)
	{}

	static void Show(TSharedPtr<SWindow> InParentWindow)
	{
		if(!ActiveNotification.IsValid())
		{
			TSharedRef<SNewUserTipNotification> ActiveNotificationRef = 
				SNew(SNewUserTipNotification);

			ActiveNotification = ActiveNotificationRef;
			ParentWindow = InParentWindow;
			InParentWindow->AddOverlaySlot()
				.VAlign(VAlign_Bottom)
				.HAlign(HAlign_Left)
				.Padding(FMargin(20.0f, 20.0f, 10.0f, 50.f))
				[
					ActiveNotificationRef
				];
		}

	}

	static void Dismiss()
	{
		TSharedPtr<SNewUserTipNotification> ActiveNotificationPin = ActiveNotification.Pin();
		if (ParentWindow.IsValid() && ActiveNotificationPin.IsValid())
		{
			ParentWindow.Pin()->RemoveOverlaySlot(ActiveNotificationPin.ToSharedRef());
		}

		ParentWindow.Reset();

		ActiveNotification.Reset();
	}

	void Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			SNew(SBox)
			.WidthOverride(350.0f)
			.HeightOverride(128.0f)
			[
				SNew(SNotificationBackground)
				.Padding(FMargin(16, 8))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(0.0f, 6.0f, 0.0f, 0.0f)
					.VAlign(VAlign_Top)
					.AutoWidth()
					[
						SNew(SBorder)
						.Padding(FMargin(11,4))
						.BorderImage(&NewBadgeBrush)
						.ForegroundColor(FStyleColors::ForegroundInverted)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("NewBadge", "New"))
							.TextStyle(FAppStyle::Get(), "SmallButtonText")
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(16.0f, 8.0f, 0.0f, 0.0f)
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 0.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
							.Font(FAppStyle::Get().GetFontStyle("NotificationList.FontBold"))
							.Text(LOCTEXT("ContentDrawerTipTitle", "Content Drawer"))
							.ColorAndOpacity(FStyleColors::ForegroundHover)
						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 12.0f, 0.0f, 0.0f)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SBorder)
								.Padding(FMargin(20, 4))
								.BorderImage(&KeybindBackgroundBrush)
								[
									SNew(STextBlock)
									.TextStyle(FAppStyle::Get(), "DialogButtonText")
									.Text(FGlobalEditorCommonCommands::Get().OpenContentBrowserDrawer->GetActiveChord(EMultipleKeyBindingIndex::Primary)->GetModifierText(FText::GetEmpty()))
									.ColorAndOpacity(FStyleColors::ForegroundHover)
								]
							]
							+ SHorizontalBox::Slot()
							.Padding(8.0f, 0.0f)
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(SImage)
								.Image(FAppStyle::Get().GetBrush("Icons.Plus"))
								.ColorAndOpacity(FStyleColors::ForegroundHover)
							]
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SBorder)
								.Padding(FMargin(20, 4))
								.BorderImage(&KeybindBackgroundBrush)
								[
									SNew(STextBlock)
									.TextStyle(FAppStyle::Get(), "DialogButtonText")
									.Text(FGlobalEditorCommonCommands::Get().OpenContentBrowserDrawer->GetActiveChord(EMultipleKeyBindingIndex::Primary)->Key.GetDisplayName(false))
									.ColorAndOpacity(FStyleColors::ForegroundHover)
								]
							]
						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.0f, 12.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ContentDrawerTipDesc", "Summon the content browser in\ncollapsable drawer."))
							.ColorAndOpacity(FStyleColors::Foreground)
						]
					]
					+ SHorizontalBox::Slot()
					.Padding(0.0f, 0.0f, 0.0f, 0.0f)
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Top)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.OnClicked_Lambda([]() { SNewUserTipNotification::Dismiss(); return FReply::Handled(); })
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.X"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
				]
			]
		];
	}

private:
	FSlateRoundedBoxBrush NewBadgeBrush;
	FSlateRoundedBoxBrush KeybindBackgroundBrush;
	static TWeakPtr<SNewUserTipNotification> ActiveNotification;
	static TWeakPtr<SWindow> ParentWindow;
};

TWeakPtr<SNewUserTipNotification> SNewUserTipNotification::ActiveNotification;
TWeakPtr<SWindow> SNewUserTipNotification::ParentWindow;


void UStatusBarSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	FSourceControlCommands::Register();

	IMainFrameModule& MainFrameModule = IMainFrameModule::Get();
	if (MainFrameModule.IsWindowInitialized())
	{
		CreateAndShowNewUserTipIfNeeded(MainFrameModule.GetParentWindow(), false);
		CreateAndShowOneTimeIndustryQueryIfNeeded(MainFrameModule.GetParentWindow(), false);
	}
	else
	{
		MainFrameModule.OnMainFrameCreationFinished().AddUObject(this, &UStatusBarSubsystem::CreateAndShowNewUserTipIfNeeded);
		MainFrameModule.OnMainFrameCreationFinished().AddUObject(this, &UStatusBarSubsystem::CreateAndShowOneTimeIndustryQueryIfNeeded);
	}


	FSlateNotificationManager::Get().SetProgressNotificationHandler(this);
}

void UStatusBarSubsystem::Deinitialize()
{
	FSourceControlCommands::Unregister();

	FSlateNotificationManager::Get().SetProgressNotificationHandler(nullptr);

	StatusBarContentBrowser.Reset();
	StatusBarOutputLog.Reset();
}

bool UStatusBarSubsystem::ToggleDebugConsole(TSharedRef<SWindow> ParentWindow, bool bAlwaysToggleDrawer)
{
	bool bToggledSuccessfully = false;

	FOutputLogModule& OutputLogModule = FModuleManager::Get().LoadModuleChecked<FOutputLogModule>("OutputLog");


	// Get the global output log tab if it exists. If it exists and is in the same editor as the status bar we'll focus that instead
	TSharedPtr<SDockTab> MainOutputLogTab = OutputLogModule.GetOutputLogTab();
	
	const bool bCycleToOutputLogDrawer = OutputLogModule.ShouldCycleToOutputLogDrawer() || bAlwaysToggleDrawer;
	
	for (const TPair<FName, FStatusBarData>& StatusBar : StatusBars)
	{
		const FStatusBarData& SBData = StatusBar.Value;
		if (TSharedPtr<SStatusBar> StatusBarPinned = SBData.StatusBarWidget.Pin())
		{
			TSharedPtr<SDockTab> ParentTab = StatusBarPinned->GetParentTab();
			if (ParentTab && ParentTab->IsForeground() && ParentTab->GetParentWindow() == ParentWindow)
			{
				FWidgetPath ConsoleEditBoxPath;
				if(FSlateApplication::Get().GeneratePathToWidgetUnchecked(SBData.ConsoleEditBox.ToSharedRef(), ConsoleEditBoxPath))
				{
					if (bAlwaysToggleDrawer && MainOutputLogTab && MainOutputLogTab->GetTabManagerPtr() && MainOutputLogTab->GetTabManagerPtr()->GetOwnerTab() == ParentTab)
					{
						OutputLogModule.FocusOutputLogConsoleBox(OutputLogModule.GetOutputLog().ToSharedRef());
					}
					else
					{
						// This toggles between 3 states: 
						// If the drawer is opened, close it
						// if the console edit box is focused, open the drawer
						// if something else is focused, focus the console edit box
						if (StatusBarPinned->IsDrawerOpened(StatusBarDrawerIds::OutputLog))
						{
							StatusBarPinned->DismissDrawer(nullptr);
						}
						else if (SBData.ConsoleEditBox->HasKeyboardFocus() || bAlwaysToggleDrawer)
						{
							if (bCycleToOutputLogDrawer)
							{
								StatusBarPinned->OpenDrawer(StatusBarDrawerIds::OutputLog);
							}
							else
							{
								// Restore previous focus
								FSlateApplication::Get().SetKeyboardFocus(PreviousKeyboardFocusedWidget.Pin(), EFocusCause::SetDirectly);
								PreviousKeyboardFocusedWidget.Reset();
							}
						}
						else
						{
							// Cache off the previously focused widget so we can restore focus if the user hits the focus key again
							PreviousKeyboardFocusedWidget = FSlateApplication::Get().GetKeyboardFocusedWidget();
							FSlateApplication::Get().SetKeyboardFocus(ConsoleEditBoxPath, EFocusCause::SetDirectly);
						}
					}

					bToggledSuccessfully = true;
					break;
				}
			}
		}
	}

	return bToggledSuccessfully;
}

bool UStatusBarSubsystem::OpenContentBrowserDrawer()
{
	return TriggerContentBrowser(EDrawerTriggerMode::Open);
}

bool UStatusBarSubsystem::ToggleContentBrowserDrawer()
{
	return TriggerContentBrowser(EDrawerTriggerMode::Toggle);
}

bool UStatusBarSubsystem::DismissContentBrowserDrawer()
{
	return TriggerContentBrowser(EDrawerTriggerMode::Dismiss);
}

bool UStatusBarSubsystem::OpenOutputLogDrawer()
{
	TSharedPtr<SWindow> ParentWindow = UE::StatusBarSubsystem::Private::FindParentWindowBehindNotification();
	
	if (ParentWindow.IsValid() && ParentWindow->GetType() == EWindowType::Normal)
	{
		constexpr bool bAlwaysToggleDrawer = true;
		return ToggleDebugConsole(ParentWindow.ToSharedRef(), bAlwaysToggleDrawer);
	}

	return false;
}

bool UStatusBarSubsystem::TryToggleDrawer(const FName DrawerId)
{
	bool bToggledSuccessfully = false;

	TSharedPtr<SWindow> ParentWindow = UE::StatusBarSubsystem::Private::FindParentWindowBehindNotification();

	if (ParentWindow.IsValid() && ParentWindow->GetType() == EWindowType::Normal)
	{
		for (const TPair<FName, FStatusBarData>& StatusBar : StatusBars)
		{
			const FStatusBarData& SBData = StatusBar.Value;
			if (TSharedPtr<SStatusBar> StatusBarPinned = SBData.StatusBarWidget.Pin())
			{
				TSharedPtr<SDockTab> ParentTab = StatusBarPinned->GetParentTab();
				if (ParentTab && ParentTab->IsForeground() && ParentTab->GetParentWindow() == ParentWindow)
				{
					if (StatusBarPinned->IsDrawerOpened(DrawerId))
					{
						StatusBarPinned->DismissDrawer(nullptr);
					}
					else
					{
						StatusBarPinned->OpenDrawer(DrawerId);
					}

					bToggledSuccessfully = true;
					break;
				}
			}
		}
	}

	return bToggledSuccessfully;
}

bool UStatusBarSubsystem::ForceDismissDrawer()
{
	bool bWasDismissed = false;
	for (const TPair<FName, FStatusBarData>& StatusBar : StatusBars)
	{
		if (TSharedPtr<SStatusBar> StatusBarPinned = StatusBar.Value.StatusBarWidget.Pin())
		{
			bWasDismissed |= StatusBarPinned->DismissDrawer(nullptr);
		}
	}

	return bWasDismissed;
}

bool UStatusBarSubsystem::TriggerContentBrowser(EDrawerTriggerMode DrawerTriggerMode)
{
	TSharedPtr<SWindow> ParentWindow = UE::StatusBarSubsystem::Private::FindParentWindowBehindNotification();

	if (!ParentWindow)
	{
		return false;
	}

	bool bWasAlreadyOpened = false;

	SNewUserTipNotification::Dismiss();

	for (const TPair<FName, FStatusBarData>& StatusBar : StatusBars)
	{
		if (TSharedPtr<SStatusBar> StatusBarPinned = StatusBar.Value.StatusBarWidget.Pin())
		{
			if(StatusBarPinned->IsDrawerOpened(StatusBarDrawerIds::ContentBrowser))
			{
				TSharedPtr<SDockTab> ParentTab = StatusBarPinned->GetParentTab();
				if (ParentTab && ParentTab->IsForeground() && ParentTab->GetParentWindow() == ParentWindow)
				{
					if (EnumHasAllFlags(DrawerTriggerMode, EDrawerTriggerMode::Dismiss))
					{
						StatusBarPinned->DismissDrawer(nullptr);
					}

					bWasAlreadyOpened = true;
				}
			}
		}
	}

	if(!bWasAlreadyOpened && EnumHasAllFlags(DrawerTriggerMode, EDrawerTriggerMode::Open))
	{
		TSharedPtr<SWindow> Window = ParentWindow;
		GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateUObject(this, &UStatusBarSubsystem::HandleDeferredOpenContentBrowser, Window));
	}

	return true;
}

TSharedRef<SWidget> UStatusBarSubsystem::MakeStatusBarWidget(FName StatusBarName, const TSharedRef<SDockTab>& InParentTab, UE::Editor::Toolbars::ECreateStatusBarOptions CreateStatusBarOptions)
{
	using namespace UE::Editor::Toolbars;

	LLM_SCOPE(ELLMTag::UI);

	CreateContentBrowserIfNeeded();

	TArray<UStatusBarSubsystem::FTabIdAndButtonLabel> PanelDrawerSummons = UE::StatusBarSubsystem::Private::CreatePanelDrawerSummonsData(InParentTab, RegisteredPanelDrawerSummons);

	TSharedRef<SStatusBar> StatusBar =
		SNew(SStatusBar, StatusBarName, InParentTab, MoveTemp(PanelDrawerSummons))
		.CreateSourceControlSection(!EnumHasAnyFlags(CreateStatusBarOptions, ECreateStatusBarOptions::HideSourceControl));

	if (!EnumHasAnyFlags(CreateStatusBarOptions, UE::Editor::Toolbars::ECreateStatusBarOptions::HideContentBrowser))
	{
		FWidgetDrawerConfig ContentBrowserDrawer(StatusBarDrawerIds::ContentBrowser);
		ContentBrowserDrawer.GetDrawerContentDelegate.BindUObject(this, &UStatusBarSubsystem::OnGetContentBrowser);
		ContentBrowserDrawer.OnDrawerOpenedDelegate.BindUObject(this, &UStatusBarSubsystem::OnContentBrowserOpened);
		ContentBrowserDrawer.OnDrawerDismissedDelegate.BindUObject(this, &UStatusBarSubsystem::OnContentBrowserDismissed);
		ContentBrowserDrawer.ButtonText = LOCTEXT("StatusBar_ContentBrowserButton", "Content Drawer");
		ContentBrowserDrawer.ToolTipText = FText::Format(LOCTEXT("StatusBar_ContentBrowserDrawerToolTip", "Opens a temporary content browser above this status which will dismiss when it loses focus ({0})"), FGlobalEditorCommonCommands::Get().OpenContentBrowserDrawer->GetInputText());
		ContentBrowserDrawer.Icon = FAppStyle::Get().GetBrush("ContentBrowser.TabIcon");

		for (const TUniquePtr<IGlobalStatusBarExtension>& Extension : GlobalStatusBarExtensions)
		{
			Extension->ExtendContentBrowserDrawer(ContentBrowserDrawer);
		}

		StatusBar->RegisterDrawer(MoveTemp(ContentBrowserDrawer));
	}

	FOutputLogModule& OutputLogModule = FModuleManager::Get().LoadModuleChecked<FOutputLogModule>("OutputLog");

	TWeakPtr<SStatusBar> StatusBarWeakPtr = StatusBar;

	TSharedPtr<SMultiLineEditableTextBox> ConsoleEditBox;
	if (!EnumHasAnyFlags(CreateStatusBarOptions, UE::Editor::Toolbars::ECreateStatusBarOptions::HideOutputLog))
	{
		FSimpleDelegate OnConsoleClosed = FSimpleDelegate::CreateUObject(this, &UStatusBarSubsystem::OnDebugConsoleClosed, StatusBarWeakPtr);
		FSimpleDelegate OnConsoleCommandExecuted;

		
		TSharedPtr<SWidget> OutputLog;
		{
			TSharedRef<SWidget> ConsoleInputBox = OutputLogModule.MakeConsoleInputBox(ConsoleEditBox, OnConsoleClosed, OnConsoleCommandExecuted);

			auto IsConsoleInputBoxBorderVisible = [ConsoleInputBoxWeak = ConsoleInputBox.ToWeakPtr()]()
				{
					if (TSharedPtr<SWidget> ConsoleInputBox = ConsoleInputBoxWeak.Pin())
					{
						FSlateAttributeMetaData::UpdateOnlyVisibilityAttributes(*ConsoleInputBox, FSlateAttributeMetaData::EInvalidationPermission::AllowInvalidationIfConstructed);
						if (ConsoleInputBox->GetVisibility() == EVisibility::Collapsed)
						{
							return EVisibility::Collapsed;
						}
					}
					return EVisibility::Visible;
				};

			OutputLog =
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
				.VAlign(VAlign_Center)
				.Padding(FMargin(6.0f, 0.0f))
				.Visibility(MakeAttributeLambda(IsConsoleInputBoxBorderVisible))
				[
					SNew(SBox)
						[
							ConsoleInputBox
						]
				];
		}

		FWidgetDrawerConfig OutputLogDrawer(StatusBarDrawerIds::OutputLog);

		OutputLogDrawer.GetDrawerContentDelegate.BindUObject(this, &UStatusBarSubsystem::OnGetOutputLog);
		OutputLogDrawer.OnDrawerOpenedDelegate.BindUObject(this, &UStatusBarSubsystem::OnOutputLogOpened);
		OutputLogDrawer.OnDrawerDismissedDelegate.BindUObject(this, &UStatusBarSubsystem::OnOutputLogDismised);
		OutputLogDrawer.CustomWidget = OutputLog;

		OutputLogDrawer.ButtonText = LOCTEXT("StatusBar_OutputLogButton", "Output Log");
		OutputLogDrawer.ToolTipText = FText::Format(LOCTEXT("StatusBar_OutputLogButtonTip", "Opens the output log drawer. ({0}) cycles between focusing the console command box, opening the output log drawer, and closing it.\nThe output log drawer may also be toggled directly with ({1})"), FGlobalEditorCommonCommands::Get().OpenConsoleCommandBox->GetInputText(), FGlobalEditorCommonCommands::Get().OpenOutputLogDrawer->GetInputText());
		OutputLogDrawer.Icon = FAppStyle::Get().GetBrush("Log.TabIcon");

		for (const TUniquePtr<IGlobalStatusBarExtension>& Extension : GlobalStatusBarExtensions)
		{
			Extension->ExtendOutputLogDrawer(OutputLogDrawer);
		}

		StatusBar->RegisterDrawer(MoveTemp(OutputLogDrawer));
	}


	// Clean up stale status bars
	for (auto It = StatusBars.CreateIterator(); It; ++It)
	{
		if (!It.Value().StatusBarWidget.IsValid())
		{
			It.RemoveCurrent();
		}
	}

	FStatusBarData StatusBarData;
	StatusBarData.StatusBarWidget = StatusBar;
	StatusBarData.ConsoleEditBox = ConsoleEditBox ? ConsoleEditBox : SNullWidget::NullWidget.ToSharedPtr();

	StatusBars.Add(StatusBarName, StatusBarData);

	return StatusBar;
}

bool UStatusBarSubsystem::ActiveWindowHasStatusBar() const
{
	TSharedPtr<SWindow> ParentWindow = UE::StatusBarSubsystem::Private::FindParentWindow();

	if (ParentWindow.IsValid() && ParentWindow->GetType() == EWindowType::Normal)
	{
		for (const TPair<FName, FStatusBarData>& StatusBar : StatusBars)
		{
			if (TSharedPtr<SStatusBar> StatusBarPinned = StatusBar.Value.StatusBarWidget.Pin())
			{
				TSharedPtr<SDockTab> ParentTab = StatusBarPinned->GetParentTab();
				if (ParentTab && ParentTab->IsForeground() && ParentTab->GetParentWindow() == ParentWindow)
				{
					return true;
				}
			}
		}
	}

	return false;
}

// This function is identical to UStatusBarSubsystem::ActiveWindowHasStatusBar(), except the variable ParentWindow stores the topmost
// Regular window rather than the direct parent window. This function is only used when users click on the hyperlink "Show Output Log" on 
// the notification window. 
bool UStatusBarSubsystem::ActiveWindowBehindNotificationHasStatusBar()
{
	TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().GetActiveTopLevelRegularWindow(); 

	// Same code as ActiveWindowHasStatusBar() from here: 
	if (ParentWindow.IsValid() && ParentWindow->GetType() == EWindowType::Normal)
	{
		for (const TPair<FName, FStatusBarData>& StatusBar : StatusBars)
		{
			if (TSharedPtr<SStatusBar> StatusBarPinned = StatusBar.Value.StatusBarWidget.Pin())
			{
				TSharedPtr<SDockTab> ParentTab = StatusBarPinned->GetParentTab();
				if (ParentTab && ParentTab->IsForeground() && ParentTab->GetParentWindow() == ParentWindow)
				{
					return true;
				}
			}
		}
	}
	return false;
}

void UStatusBarSubsystem::RegisterDrawer(FName StatusBarName, FWidgetDrawerConfig&& Drawer, int32 SlotIndex)
{
	if (TSharedPtr<SStatusBar> StatusBar = GetStatusBar(StatusBarName))
	{
		StatusBar->RegisterDrawer(MoveTemp(Drawer), SlotIndex);
	}
}

void UStatusBarSubsystem::UnregisterDrawer(FName StatusBarName, FName DrawerId)
{
	if (TSharedPtr<SStatusBar> StatusBar = GetStatusBar(StatusBarName))
	{
		StatusBar->UnregisterDrawer(DrawerId);
	}
}

FStatusBarMessageHandle UStatusBarSubsystem::PushStatusBarMessage(FName StatusBarName, const TAttribute<FText>& InMessage, const TAttribute<FText>& InHintText)
{
	if (TSharedPtr<SStatusBar> StatusBar = GetStatusBar(StatusBarName))
	{
		FStatusBarMessageHandle NewHandle(++MessageHandleCounter);

		StatusBar->PushMessage(NewHandle, InMessage, InHintText);

		return NewHandle;
	}

	return FStatusBarMessageHandle();
}

FStatusBarMessageHandle UStatusBarSubsystem::PushStatusBarMessage(FName StatusBarName, const TAttribute<FText>& InMessage)
{
	return PushStatusBarMessage(StatusBarName, InMessage, TAttribute<FText>());
}

FStatusBarMessageHandle UStatusBarSubsystem::PushStatusBarMessage(FName StatusBarName, const TAttribute<FText>& InMessage, const TAttribute<FText>& InHintText,
	const TAttribute<const FSlateBrush*>& InIconBrush, const TAttribute<FSlateColor>& InIconColor)
{
	if (TSharedPtr<SStatusBar> StatusBar = GetStatusBar(StatusBarName))
	{
		FStatusBarMessageHandle NewHandle(++MessageHandleCounter);

		StatusBar->PushMessage(NewHandle, InMessage, InHintText, InIconBrush, InIconColor);

		return NewHandle;
	}

	return FStatusBarMessageHandle();
}

void UStatusBarSubsystem::PopStatusBarMessage(FName StatusBarName, FStatusBarMessageHandle InHandle)
{
	if (TSharedPtr<SStatusBar> StatusBar = GetStatusBar(StatusBarName))
	{
		StatusBar->PopMessage(InHandle);
	}
}

void UStatusBarSubsystem::ClearStatusBarMessages(FName StatusBarName)
{
	if (TSharedPtr<SStatusBar> StatusBar = GetStatusBar(StatusBarName))
	{
		StatusBar->ClearAllMessages();
	}
}

IGlobalStatusBarExtension& UStatusBarSubsystem::RegisterGlobalStatusBarExtension(TUniquePtr<IGlobalStatusBarExtension>&& Extension)
{
	int32 Index = GlobalStatusBarExtensions.Add(MoveTemp(Extension));
	// NOTE: It is safe to return this reference because it's stored in a TUniquePtr which is
	// guaranteed not to change its address if the array reallocates.
	return *GlobalStatusBarExtensions[Index];
}

TUniquePtr<IGlobalStatusBarExtension> UStatusBarSubsystem::UnregisterGlobalStatusBarExtension(IGlobalStatusBarExtension* Extension)
{
	int32 Index = GlobalStatusBarExtensions.IndexOfByPredicate([Extension](const TUniquePtr<IGlobalStatusBarExtension>& Other)
	{
		return &*Other == Extension;
	});
	TUniquePtr<IGlobalStatusBarExtension> RemovedExtension = MoveTemp(GlobalStatusBarExtensions[Index]);
	GlobalStatusBarExtensions.RemoveAtSwap(Index);
	return RemovedExtension;
}

FDelegateHandle UStatusBarSubsystem::RegisterPanelDrawerSummon(FRegisterPanelDrawerSummonDelegate::FDelegate&& InRegistrationCallback)
{
	return RegisteredPanelDrawerSummons.Add(MoveTemp(InRegistrationCallback));
}
void UStatusBarSubsystem::UnregisterPanelDrawerSummon(FDelegateHandle InHandle)
{
	RegisteredPanelDrawerSummons.Remove(InHandle);
}

void UStatusBarSubsystem::StartProgressNotification(FProgressNotificationHandle Handle, FText DisplayText, int32 TotalWorkToDo)
{
	// Avoid crashing when starting progress notification while slate is still uninitialized. (i.e. commandlet)
	if (FSlateApplication::IsInitialized())
	{
		// Get the active window, if one is not active a notification was started when the application was deactivated so use the focus path to find a window or just use the root window if there is no keyboard focus
		TSharedPtr<SWindow> ActiveWindow = FSlateApplication::Get().GetActiveTopLevelRegularWindow();
		if (!ActiveWindow)
		{
			TSharedPtr<SWidget> FocusedWidget = FSlateApplication::Get().GetKeyboardFocusedWidget();
			ActiveWindow = FocusedWidget ? FSlateApplication::Get().FindWidgetWindow(FocusedWidget.ToSharedRef()) : FGlobalTabmanager::Get()->GetRootWindow();
		}

		// Find the active status bar to display the progress in
		for (const TPair<FName, FStatusBarData>& StatusBar : StatusBars)
		{
			if (TSharedPtr<SStatusBar> StatusBarPinned = StatusBar.Value.StatusBarWidget.Pin())
			{
				TSharedPtr<SDockTab> ParentTab = StatusBarPinned->GetParentTab();
				if (ParentTab && ParentTab->IsForeground() && ParentTab->GetParentWindow() == ActiveWindow)
				{
					StatusBarPinned->StartProgressNotification(Handle, DisplayText, TotalWorkToDo);
					break;
				}
			}
		}
	}
}

void UStatusBarSubsystem::UpdateProgressNotification(FProgressNotificationHandle Handle, int32 TotalWorkDone, int32 UpdatedTotalWorkToDo, FText UpdatedDisplayText)
{
	for (const TPair<FName, FStatusBarData>& StatusBar : StatusBars)
	{
		if (TSharedPtr<SStatusBar> StatusBarPinned = StatusBar.Value.StatusBarWidget.Pin())
		{
			if (StatusBarPinned->UpdateProgressNotification(Handle, TotalWorkDone, UpdatedTotalWorkToDo, UpdatedDisplayText))
			{
				break;
			}
		}
	}

}

void UStatusBarSubsystem::CancelProgressNotification(FProgressNotificationHandle Handle)
{
	for (const TPair<FName, FStatusBarData>& StatusBar : StatusBars)
	{
		if (TSharedPtr<SStatusBar> StatusBarPinned = StatusBar.Value.StatusBarWidget.Pin())
		{
			if (StatusBarPinned->CancelProgressNotification(Handle))
			{
				break;
			}
		}
	}
}

void UStatusBarSubsystem::OnDebugConsoleClosed(TWeakPtr<SStatusBar> OwningStatusBar)
{
	if (TSharedPtr<SStatusBar> OwningStatusBarPinned = OwningStatusBar.Pin())
	{
		TSharedPtr<SWindow> OwningWindow = OwningStatusBarPinned->GetParentTab()->GetParentWindow();
		ToggleDebugConsole(OwningWindow.ToSharedRef());
	}
}

void UStatusBarSubsystem::CreateContentBrowserIfNeeded()
{
	if(!StatusBarContentBrowser.IsValid())
	{
		IContentBrowserSingleton& ContentBrowserSingleton = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();;

		FContentBrowserConfig Config;
		Config.bCanSetAsPrimaryBrowser = true;

		TFunction<TSharedPtr<SDockTab>()> GetTab(
			[this]() -> TSharedPtr<SDockTab>
			{
				UE_LOGF(LogStatusBar, Log, "Looking status bar with open content browser drawer...")
				for (const TPair<FName, FStatusBarData>& StatusBar : StatusBars)
				{
					if (TSharedPtr<SStatusBar> StatusBarPinned = StatusBar.Value.StatusBarWidget.Pin())
					{
						if (StatusBarPinned->IsDrawerOpened(StatusBarDrawerIds::ContentBrowser))
						{
							UE_LOGF(LogStatusBar, Log, "Using status bar: %ls to dock content browser", *StatusBar.Key.ToString());
							return StatusBarPinned->GetParentTab();
						}
						else
						{
							UE_LOGF(LogStatusBar, Log, "StatusBar: %ls was content browser was not opened", *StatusBar.Key.ToString());
						}
					}
				}

				ensureMsgf(false, TEXT("If we get here somehow a content browser drawer is opened but no status bar claims it"));
				return TSharedPtr<SDockTab>();
			}
		);
		StatusBarContentBrowser = ContentBrowserSingleton.CreateContentBrowserDrawer(Config, GetTab);
	}
}

void UStatusBarSubsystem::CreateAndShowNewUserTipIfNeeded(TSharedPtr<SWindow> ParentWindow, bool bIsRunningStartupDialog)
{
	if(!bIsRunningStartupDialog)
	{
		FString CurrentState = GetNewUserTipState();

		if(CurrentState != TEXT("1"))
		{
			SNewUserTipNotification::Show(ParentWindow);

			const FString StoreId = TEXT("Epic Games");
			const FString SectionName = TEXT("Unreal Engine/Editor");
			const FString KeyName = TEXT("LaunchTipShown");

			const FString FallbackIniLocation = TEXT("/Script/UnrealEd.EditorSettings");
			const FString FallbackIniKey = TEXT("LaunchTipShownFallback");
			// Write that we've shown the notification
			UStatusBarSubsystem::SetOneTimeStateWithFallback(StoreId, SectionName, KeyName, FallbackIniLocation, FallbackIniKey);
		}
	}

	// Ignore the if the main frame gets recreated this session
	IMainFrameModule::Get().OnMainFrameCreationFinished().RemoveAll(this);
}

const FString UStatusBarSubsystem::GetNewUserTipState() const
{
	const FString StoreId = TEXT("Epic Games");
	const FString SectionName = TEXT("Unreal Engine/Editor");
	const FString KeyName = TEXT("LaunchTipShown");

	const FString FallbackIniLocation = TEXT("/Script/UnrealEd.EditorSettings");
	const FString FallbackIniKey = TEXT("LaunchTipShownFallback");

	FString CurrentState = UStatusBarSubsystem::GetOneTimeStateWithFallback(StoreId, SectionName, KeyName, FallbackIniLocation, FallbackIniKey);
	
	return CurrentState;
}

void UStatusBarSubsystem::CreateAndShowOneTimeIndustryQueryIfNeeded(TSharedPtr<SWindow> ParentWindow, bool bIsRunningStartupDialog)
{
	// Only show for external builds where editor analytics are on and the industry popup is not suppressed
	if(SOneTimeIndustryQuery::ShouldShowIndustryQuery())
	{
		FString NewUserTipState = GetNewUserTipState();
		if (!bIsRunningStartupDialog && NewUserTipState == TEXT("1"))
		{
			const FString StoreId = TEXT("Epic Games");
			const FString SectionName = TEXT("Unreal Engine/Editor");
			const FString KeyName = TEXT("OneTimeIndustryQueryShown");

			const FString FallbackIniLocation = TEXT("/Script/UnrealEd.EditorSettings");
			const FString FallbackIniKey = TEXT("OneTimeIndustryQueryShownFallback");

			FString CurrentState = UStatusBarSubsystem::GetOneTimeStateWithFallback(StoreId, SectionName, KeyName, FallbackIniLocation, FallbackIniKey);

			if (CurrentState != TEXT("1"))
			{
				SOneTimeIndustryQuery::Show(ParentWindow);
				SetOneTimeStateWithFallback(StoreId, SectionName, KeyName, FallbackIniLocation, FallbackIniKey);
			}
		}
	}
	// Ignore this if the main frame gets recreated this session
	IMainFrameModule::Get().OnMainFrameCreationFinished().RemoveAll(this);
}

const FString UStatusBarSubsystem::GetOneTimeStateWithFallback(const FString StoreId, const FString SectionName, const FString KeyName, const FString FallbackIniLocation, const FString FallbackIniKey)
{
	// Its important that this new user message does not appear after the first launch so we store it in a more permanent place
	FString CurrentState = TEXT("0");
	if (!FPlatformMisc::GetStoredValue(StoreId, SectionName, KeyName, CurrentState))
	{
		// As a fallback where the registry was not readable or writable, save a flag in the editor ini. This will be less permanent as the registry but will prevent 
		// the notification from appearing on every launch
		GConfig->GetString(*FallbackIniLocation, *FallbackIniKey, CurrentState, GEditorSettingsIni);
	}
	return CurrentState;
}

void UStatusBarSubsystem::SetOneTimeStateWithFallback(const FString StoreId, const FString SectionName, const FString KeyName, const FString FallbackIniLocation, const FString FallbackIniKey)
{
	// Write that we've shown the notification
	if (!FPlatformMisc::SetStoredValue(StoreId, SectionName, KeyName, TEXT("1")))
	{
		// Use fallback
		GConfig->SetString(*FallbackIniLocation, *FallbackIniKey, TEXT("1"), GEditorSettingsIni);
	}
}

TSharedPtr<SStatusBar> UStatusBarSubsystem::GetStatusBar(FName StatusBarName) const
{
	return StatusBars.FindRef(StatusBarName).StatusBarWidget.Pin();
}

TSharedRef<SWidget> UStatusBarSubsystem::OnGetContentBrowser()
{
	CreateContentBrowserIfNeeded();

	return StatusBarContentBrowser.ToSharedRef();
}

void UStatusBarSubsystem::OnContentBrowserOpened(FName StatusBarWithDrawerName)
{
	SNewUserTipNotification::Dismiss();

	// Dismiss any other content browser that is opened when one status bar opens it.  The content browser is a shared resource and shouldn't be in the layout twice
	for (const TPair<FName, FStatusBarData>& StatusBar : StatusBars)
	{
		if (TSharedPtr<SStatusBar> StatusBarPinned = StatusBar.Value.StatusBarWidget.Pin())
		{
			if (StatusBarWithDrawerName != StatusBarPinned->GetStatusBarName() || StatusBarPinned->IsAnyOtherDrawerOpened(StatusBarDrawerIds::ContentBrowser))
			{
				StatusBarPinned->CloseDrawerImmediately();
			}
		}
	}

	IContentBrowserSingleton& ContentBrowserSingleton = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();;

	// Cache off the previously focused widget so we can restore focus if the user hits the focus key again
	PreviousKeyboardFocusedWidget = FSlateApplication::Get().GetKeyboardFocusedWidget();

	ContentBrowserSingleton.FocusContentBrowserSearchField(StatusBarContentBrowser);
}

void UStatusBarSubsystem::OnContentBrowserDismissed(const TSharedPtr<SWidget>& NewlyFocusedWidget)
{
	if (PreviousKeyboardFocusedWidget.IsValid() && !NewlyFocusedWidget.IsValid())
	{
		FSlateApplication::Get().SetKeyboardFocus(PreviousKeyboardFocusedWidget.Pin());
	}

	IContentBrowserSingleton& ContentBrowserSingleton = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();;
	ContentBrowserSingleton.SaveContentBrowserSettings(StatusBarContentBrowser);

	PreviousKeyboardFocusedWidget.Reset();
}

void UStatusBarSubsystem::HandleDeferredOpenContentBrowser(TSharedPtr<SWindow> ParentWindow)
{
	for (const TPair<FName, FStatusBarData>& StatusBar : StatusBars)
	{
		if (TSharedPtr<SStatusBar> StatusBarPinned = StatusBar.Value.StatusBarWidget.Pin())
		{
			TSharedPtr<SDockTab> ParentTab = StatusBarPinned->GetParentTab();
			if (ParentTab && ParentTab->IsForeground() && ParentTab->GetParentWindow() == ParentWindow)
			{
				StatusBarPinned->OpenDrawer(StatusBarDrawerIds::ContentBrowser);
				break;
			}
		}
	}
}

TSharedRef<SWidget> UStatusBarSubsystem::OnGetOutputLog()
{
	FOutputLogModule& OutputLogModule = FModuleManager::Get().LoadModuleChecked<FOutputLogModule>("OutputLog"); 

	if (!StatusBarOutputLog)
	{
		StatusBarOutputLog = OutputLogModule.MakeOutputLogDrawerWidget(FSimpleDelegate::CreateUObject(this, &UStatusBarSubsystem::OnDebugConsoleDrawerClosed));
	}

	return StatusBarOutputLog.ToSharedRef();
}

void UStatusBarSubsystem::OnOutputLogOpened(FName StatusBarWithDrawerName)
{
	// Dismiss any other content browser that is opened when one status bar opens it.  The content browser is a shared resource and shouldn't be in the layout twice
	for (const TPair<FName, FStatusBarData>& StatusBar : StatusBars)
	{
		if (TSharedPtr<SStatusBar> StatusBarPinned = StatusBar.Value.StatusBarWidget.Pin())
		{
			if (StatusBarWithDrawerName != StatusBarPinned->GetStatusBarName() || StatusBarPinned->IsAnyOtherDrawerOpened(StatusBarDrawerIds::OutputLog))
			{
				StatusBarPinned->CloseDrawerImmediately();
			}
		}
	}
	FOutputLogModule& OutputLogModule = FModuleManager::Get().LoadModuleChecked<FOutputLogModule>("OutputLog");
	OutputLogModule.FocusOutputLogConsoleBox(StatusBarOutputLog.ToSharedRef());
}

void UStatusBarSubsystem::OnOutputLogDismised(const TSharedPtr<SWidget>& NewlyFocusedWidget)
{
	if (PreviousKeyboardFocusedWidget.IsValid() && !NewlyFocusedWidget.IsValid())
	{
		FSlateApplication::Get().SetKeyboardFocus(PreviousKeyboardFocusedWidget.Pin());
	}

	PreviousKeyboardFocusedWidget.Reset();

}

void UStatusBarSubsystem::OnDebugConsoleDrawerClosed()
{
	for (const TPair<FName, FStatusBarData>& StatusBar : StatusBars)
	{
		if (TSharedPtr<SStatusBar> StatusBarPinned = StatusBar.Value.StatusBarWidget.Pin())
		{
			if (StatusBarPinned->IsDrawerOpened(StatusBarDrawerIds::OutputLog))
			{
				ToggleDebugConsole(StatusBarPinned->GetParentTab()->GetParentWindow().ToSharedRef());
				break;
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
