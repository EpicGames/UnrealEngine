// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

struct FWidgetDrawerConfig;
class SStatusBar;

/** Extension point for extending the functionality of status bars across the entire editor. */
struct IGlobalStatusBarExtension
{
	virtual ~IGlobalStatusBarExtension() = default;

	/** Called before the content browser drawer is registered in the status bar. */
	virtual void ExtendContentBrowserDrawer(FWidgetDrawerConfig& WidgetDrawerConfig) {}

	/** Called before the output log drawer is registered in the status bar. */
	virtual void ExtendOutputLogDrawer(FWidgetDrawerConfig& WidgetDrawerConfig) {}
};
