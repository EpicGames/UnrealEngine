// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


/**
 * The public interface to this module
 */
class IStatusBarModule : public IModuleInterface
{

public:
	static inline IStatusBarModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IStatusBarModule>( "StatusBar" );
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("StatusBar");
	}
};

