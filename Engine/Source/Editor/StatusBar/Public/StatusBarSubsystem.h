// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/DelegateCombinations.h"
#include "EditorSubsystem.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GlobalStatusBarExtension.h"
#include "Internationalization/Text.h"
#include "Styling/SlateColor.h"
#include "Templates/Tuple.h"
#include "UObject/NameTypes.h"
#include "StatusBarSubsystem.generated.h"

#define UE_API STATUSBAR_API

class SStatusBar;
class SWindow;
class SWidget;
class SDockTab;
struct FWidgetDrawerConfig;
struct FSlateBrush;

template<typename ObjectType> 
class TAttribute;

struct FStatusBarMessageHandle
{
	friend class UStatusBarSubsystem;

	FStatusBarMessageHandle()
		: Id(INDEX_NONE)
	{}

	bool IsValid()
	{
		return Id != INDEX_NONE;
	}

	void Reset()
	{
		Id = INDEX_NONE;
	}

	bool operator==(const FStatusBarMessageHandle& OtherHandle) const
	{
		return Id == OtherHandle.Id;
	}
private:
	FStatusBarMessageHandle(int32 InId)
		: Id(InId)
	{}

	int32 Id;
};

namespace UE::Editor::Toolbars
{
	/** Options used to specify what parts (if any) of the status bar should be hidden. */
	enum class ECreateStatusBarOptions : uint8
	{
		Default = 0,
		HideContentBrowser = 1,
		HideOutputLog = 1 << 1,
		HideSourceControl = 1 << 2
	};
	ENUM_CLASS_FLAGS(ECreateStatusBarOptions);
}

struct FStatusBarData
{
	TWeakPtr<SStatusBar> StatusBarWidget;
	TSharedPtr<SWidget> ConsoleEditBox;
};

UCLASS(MinimalAPI)
class UStatusBarSubsystem : public UEditorSubsystem, public IProgressNotificationHandler
{
	GENERATED_BODY()

public:

	/**
	 *	Prepares for use
	 */
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override final;

	/**
	 *	Internal cleanup
	 */
	UE_API virtual void Deinitialize() override final;

	/**
	 * Focuses the debug console or opens the output log drawer on the status bar for status bar residing in the passed in parent window 
	 *
	 * @param ParentWindow			The parent window of the status bar 
	 * @param bAlwaysToggleDrawer	If true, the output log drawer will be toggled without focusing the debug console first
	 * @return true if a status bar debug console was found and focused 
	 */
	UE_API bool ToggleDebugConsole(TSharedRef<SWindow> ParentWindow, bool bAlwaysToggleDrawer=false);

	/**
	 * Opens the output log drawer for a status bar residing in the active window
	 *
	 * @return true if the output log was opened
	 */
	UE_API bool OpenOutputLogDrawer();

	/**
	 * Opens the content browser drawer for a status bar residing in the active window 
	 * 
	 * @return true if the content browser was opened
	 */
	UE_API bool OpenContentBrowserDrawer();

	/**
	 * Opens or closes the content browser drawer for a status bar residing in the active window 
	 * 
	 * @return true if the content browser was toggled or false if no status bar in the active window was found
	 */
	UE_API bool ToggleContentBrowserDrawer();

	/**
	 * Closes the content browser drawer for a status bar residing in the active window 
	 * 
	 * @return true if the content browser was dismissed
	 */
	UE_API bool DismissContentBrowserDrawer();

	/**
	 * Tries to toggle the given drawer
	 */
	UE_API bool TryToggleDrawer(const FName DrawerId);

	/**
	 * Forces all drawers to dismiss. Usually it dismisses with focus. Only call this if there is some reason an open drawer would be invalid for the current state of the editor.
	 */
	UE_API bool ForceDismissDrawer();

	/** 
	 * Creates a new instance of a status bar widget
	 *
	 * @param StatusBarName	The name of the status bar for updating it later. This name must be unique. You can use the instance number of an fname to generate a unique fname for comparison but a non-unique one for serialization of status bar data. I.E all asset editors of a specific type will be saved the same but at runtime will be indentified uniquely. This is usally what you want.
	 * @param ParentTab	Parent tab of the status bar.
	 */
	UE_API TSharedRef<SWidget> MakeStatusBarWidget(FName UniqueStatusBarName, const TSharedRef<SDockTab>& InParentTab, UE::Editor::Toolbars::ECreateStatusBarOptions CreateStatusBarOptions = UE::Editor::Toolbars::ECreateStatusBarOptions::Default);

	/**
	 * @return true if a status bar was found for the active window
	 */
	UE_API bool ActiveWindowHasStatusBar() const;

	/**
	 * @return true if a status bar was found for the active window immediately behind a Notification Window
	*/
	UE_API bool ActiveWindowBehindNotificationHasStatusBar();

	/**
	 * Creates a new instance of a status bar widget
	 *
	 * @param StatusBarName	The name of the status bar to add the drawer to
	 * @param Drawer		The drawer to add to the status bar
	 * @param SlotIndex		The position at which to add the new drawer
	 */
	UE_API void RegisterDrawer(FName StatusBarName, FWidgetDrawerConfig&& Drawer, int32 SlotIndex = INDEX_NONE);

	/**
	 * Unregisters and destroys the drawer with the given DrawerId 
	 *
	 * @param StatusBarName	The name of the status bar to unregister the drawer from
	 * @param DrawerId		the unique name id of the drawer to unregister
	 */
	UE_API void UnregisterDrawer(FName StatusBarName, FName DrawerId);

	/** 
	 * Pushes a new status bar message
	 *
	 * @param StatusBarName	The name of the status bar to push messages to
	 * @param InMessage		The message to display
	 * @param InHintText	Optional hint text message.  This message will be highlighted to make it stand out
	 * @return	A handle to the message for clearing it later
	 */
	UE_API FStatusBarMessageHandle PushStatusBarMessage(FName StatusBarName, const TAttribute<FText>& InMessage, const TAttribute<FText>& InHintText);
	UE_API FStatusBarMessageHandle PushStatusBarMessage(FName StatusBarName, const TAttribute<FText>& InMessage);

	/**
	 * Pushes a new status bar message with a custom icon brush and color.
	 * Useful for contextual indicators (e.g. a red record icon during recording).
	 *
	 * @param StatusBarName The name of the status bar to push messages to
	 * @param InMessage The message to display
	 * @param InHintText Optional hint text. Highlighted to stand out
	 * @param InIconBrush Custom icon brush. Pass nullptr to use the default help icon
	 * @param InIconColor Color/opacity for the icon, supports dynamic attributes for pulsing effects
	 * @return A handle to the message for clearing it later
	 */
	UE_API FStatusBarMessageHandle PushStatusBarMessage(FName StatusBarName, const TAttribute<FText>& InMessage, const TAttribute<FText>& InHintText,
		const TAttribute<const FSlateBrush*>& InIconBrush, const TAttribute<FSlateColor>& InIconColor);

	/**
	 * Removes a message from the status bar.  When messages are removed the previous message on the stack (if any) is displayed
	 *
	 * @param StatusBarName	The name of the status bar to remove from
	 * @param InHandle		Handle to the status bar message to remove
	 */
	UE_API void PopStatusBarMessage(FName StatusBarName, FStatusBarMessageHandle InHandle);

	/**
	 * Removes all messages from the status bar
	 *
	 * @param StatusBarName	The name of the status bar to remove from
	 */
	UE_API void ClearStatusBarMessages(FName StatusBarName);

	/**
	 * Registers a new global status bar extension.
	 *
	 * @param Extension The extension to register
	 * @return A reference to the extension interface, which can be later used to unregister the extension
	 */
	UE_API IGlobalStatusBarExtension& RegisterGlobalStatusBarExtension(TUniquePtr<IGlobalStatusBarExtension>&& Extension);

	/**
	 * Unregisters an existing status bar extension.
	 *
	 * @param Extension Reference returned by RegisterGlobalStatusBarExtension containing the extension to unregister
	 * @return An owned instance of the extension that was registered, or null if the extension was not found
	 */
	UE_API TUniquePtr<IGlobalStatusBarExtension> UnregisterGlobalStatusBarExtension(IGlobalStatusBarExtension* Extension);


	UE_EXPERIMENTAL(5.7, "The panel drawer is still in development and might be removed. The api, implementation details and its form may change in near releases.")
	typedef TPair<FName, FText> FTabIdAndButtonLabel;

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	UE_EXPERIMENTAL(5.7, "The panel drawer is still in development and might be removed. The api, implementation details and its form may change in near releases.")
	DECLARE_MULTICAST_DELEGATE_TwoParams(FRegisterPanelDrawerSummonDelegate, TArray<FTabIdAndButtonLabel>& /*OutTabIdsAndLabels*/, const TSharedRef<SDockTab>& /*InParentTab*/);

	UE_EXPERIMENTAL(5.7, "The panel drawer is still in development and might be removed. The api, implementation details and its form may change in near releases.")
	UE_API FDelegateHandle RegisterPanelDrawerSummon(FRegisterPanelDrawerSummonDelegate::FDelegate&& InRegistrationCallback);
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	UE_EXPERIMENTAL(5.7, "The panel drawer is still in development and might be removed. The api, implementation details and its form may change in near releases.")
	UE_API void UnregisterPanelDrawerSummon(FDelegateHandle ResgistrationHandle);

private:
	/** IProgressNotificationHandler interface */
	UE_API virtual void StartProgressNotification(FProgressNotificationHandle Handle, FText DisplayText, int32 TotalWorkToDo) override;
	UE_API virtual void UpdateProgressNotification(FProgressNotificationHandle Handle, int32 TotalWorkDone, int32 UpdatedTotalWorkToDo, FText UpdatedDisplayText) override;
	UE_API virtual void CancelProgressNotification(FProgressNotificationHandle Handle) override;

	enum class EDrawerTriggerMode : uint8
	{
		None = 0,

		Open = 1 << 0,
		Dismiss = 1 << 1,

		Toggle = Open | Dismiss
	};
	FRIEND_ENUM_CLASS_FLAGS(EDrawerTriggerMode);

	UE_API bool TriggerContentBrowser(EDrawerTriggerMode DrawerTriggerMode);

	UE_API void OnDebugConsoleClosed(TWeakPtr<SStatusBar> OwningStatusBar);
	UE_API void CreateContentBrowserIfNeeded();
	UE_API void CreateAndShowNewUserTipIfNeeded(TSharedPtr<SWindow> ParentWindow, bool bIsRunningStartupDialog);
	UE_API const FString GetNewUserTipState() const;
	UE_API void CreateAndShowOneTimeIndustryQueryIfNeeded(TSharedPtr<SWindow> ParentWindow, bool bIsRunningStartupDialog);
	static UE_API const FString GetOneTimeStateWithFallback(const FString StoreId, const FString SectionName, const FString KeyName, const FString FallbackIniLocation, const FString FallbackIniKey);
	static UE_API void SetOneTimeStateWithFallback(const FString StoreId, const FString SectionName, const FString KeyName, const FString FallbackIniLocation, const FString FallbackIniKey);

	UE_API TSharedPtr<SStatusBar> GetStatusBar(FName StatusBarName) const;
	UE_API TSharedRef<SWidget> OnGetContentBrowser();
	UE_API void OnContentBrowserOpened(FName StatusBarWithDrawerName);
	UE_API void OnContentBrowserDismissed(const TSharedPtr<SWidget>& NewlyFocusedWidget);
	UE_API void HandleDeferredOpenContentBrowser(TSharedPtr<SWindow> ParentWindow);

	UE_API TSharedRef<SWidget> OnGetOutputLog();
	UE_API void OnOutputLogOpened(FName StatusBarWithDrawerName);
	UE_API void OnOutputLogDismised(const TSharedPtr<SWidget>& NewlyFocusedWidget);

	UE_API void OnDebugConsoleDrawerClosed();
private:
	TMap<FName, FStatusBarData> StatusBars;
	TWeakPtr<SWidget> PreviousKeyboardFocusedWidget;
	/** The floating content browser that is opened via the content browser button in the status bar */
	TSharedPtr<SWidget> StatusBarContentBrowser;
	TSharedPtr<SWidget> StatusBarOutputLog;
	static UE_API int32 MessageHandleCounter;
	TArray<TUniquePtr<IGlobalStatusBarExtension>> GlobalStatusBarExtensions;

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	FRegisterPanelDrawerSummonDelegate RegisteredPanelDrawerSummons;
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
};

#undef UE_API
