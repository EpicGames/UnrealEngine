// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class StatusBar : ModuleRules
{
	public StatusBar(ReadOnlyTargetRules Target)
		 : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"EditorFramework",
				"EditorStyle",
				"EditorSubsystem",
				"Engine",
				"InputCore",
				"Slate",
				"SlateCore",
				"InputCore",
				"EditorFramework",
				"SourceControlWindows",
				"UnsavedAssetsTracker",
				"ToolMenus",
				"ToolWidgets",
				"UnrealEd",
				"AssetTools",
				"SourceControl"
			});

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"MainFrame",
				"OutputLog",
				"ContentBrowser", 
			});

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"MainFrame",
				"OutputLog",
				"ContentBrowser",
			});
	}
}
